// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_WidgetPreview.h"

#include "Editor.h"
#include "Engine/Blueprint.h"
#include "WidgetPreview.h"
#include "WidgetPreviewEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_WidgetPreview)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UAssetDefinition_WidgetPreview::UAssetDefinition_WidgetPreview() = default;

UAssetDefinition_WidgetPreview::~UAssetDefinition_WidgetPreview() = default;

FText UAssetDefinition_WidgetPreview::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_WidgetPreview", "Widget Preview");
}

FLinearColor UAssetDefinition_WidgetPreview::GetAssetColor() const
{
	return FLinearColor(FColor(44, 89, 180));
}

TSoftClassPtr<> UAssetDefinition_WidgetPreview::GetAssetClass() const
{
	return UWidgetPreview::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_WidgetPreview::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::UI, LOCTEXT("WidgetPreview_SubMenu", "Widget"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_WidgetPreview::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EAssetCommandResult Result = EAssetCommandResult::Unhandled;

	for (UWidgetPreview* WidgetPreview : OpenArgs.LoadObjects<UWidgetPreview>())
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		UWidgetPreviewEditor* AssetEditor = NewObject<UWidgetPreviewEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->Initialize(WidgetPreview);
		Result = EAssetCommandResult::Handled;
	}

	return Result;
}

FText UAssetDefinition_WidgetPreview::GetAssetDescription(const FAssetData& AssetData) const
{
	FString Description = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintDescription));
	if (!Description.IsEmpty())
	{
		Description.ReplaceInline( TEXT( "\\n" ), TEXT( "\n" ) );
		return FText::FromString( MoveTemp(Description) );
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
