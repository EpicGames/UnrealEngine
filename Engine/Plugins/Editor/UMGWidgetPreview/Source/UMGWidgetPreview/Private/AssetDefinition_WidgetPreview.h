// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_WidgetPreview.generated.h"

UCLASS()
class UAssetDefinition_WidgetPreview
	: public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	UAssetDefinition_WidgetPreview();
	virtual ~UAssetDefinition_WidgetPreview() override;

	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	//~ End UAssetDefinition
};
