// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewableWidgetCustomization.h"

#include "Blueprint/UserWidget.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "Styling/StyleColors.h"
#include "WidgetPreview.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "PreviewableWidgetCustomization"

namespace UE::UMGWidgetPreview::Private
{
	TSharedRef<IPropertyTypeCustomization> FPreviewableWidgetCustomization::MakeInstance()
	{
		TSharedRef<FPreviewableWidgetCustomization> Customization = MakeShared<FPreviewableWidgetCustomization>();
		return Customization;
	}

	void FPreviewableWidgetCustomization::CustomizeHeader(
		TSharedRef<IPropertyHandle> PropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		PropertyUtilities = CustomizationUtils.GetPropertyUtilities();
		PreviewVariantHandle = PropertyHandle;
		ObjectPathHandle = PreviewVariantHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPreviewableWidgetVariant, ObjectPath));

		TArray<UObject*> OwningObjects;
		PropertyHandle->GetOuterObjects(OwningObjects);
		if (!OwningObjects.IsEmpty())
		{
			if (UWidgetPreview* OwningPreview = Cast<UWidgetPreview>(OwningObjects[0]))
			{
				OwningPreview->OnWidgetChanged().AddSP(this, &FPreviewableWidgetCustomization::OnWidgetChanged);
				WeakOwningPreview = OwningPreview;
			}
		}
	}

	void FPreviewableWidgetCustomization::CustomizeChildren(
		TSharedRef<IPropertyHandle> PropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		check(ObjectPathHandle.IsValid() && ObjectPathHandle->IsValidHandle());

		ChildBuilder.AddProperty(ObjectPathHandle.ToSharedRef());

		TArray<UFunction*> CallInEditorFunctions;
		if (GetCallInEditorFunctions(CallInEditorFunctions))
		{
			AddCallInEditorFunctions(ChildBuilder, CallInEditorFunctions);
		}
	}

	bool FPreviewableWidgetCustomization::GetCallInEditorFunctions(TArray<UFunction*>& OutCallInEditorFunctions) const
	{
		check(PreviewVariantHandle.IsValid() && PreviewVariantHandle->IsValidHandle());

		FPreviewableWidgetVariant* PreviewableWidgetVariant = GetPreviewableWidgetVariant();
		if (!PreviewableWidgetVariant
			|| (PreviewableWidgetVariant && PreviewableWidgetVariant->ObjectPath.IsNull()))
		{
			return false;
		}
		
		// metadata tag for defining sort order of function buttons within a Category
		static const FName NAME_DisplayPriority("DisplayPriority");

		const bool bDisallowEditorUtilityBlueprintFunctions = GetDefault<UBlueprintEditorProjectSettings>()->bDisallowEditorUtilityBlueprintFunctionsInDetailsView;

		auto CanDisplayAndCallFunction = [](const UFunction* TestFunction)
		{
			bool bCanCall = TestFunction->GetBoolMetaData(FBlueprintMetadata::MD_CallInEditor)
						&& TestFunction->HasAnyFunctionFlags(FUNC_Public)
						&& TestFunction->ParmsSize == 0; // Params not supported

			return bCanCall;
		};
		
		if (const UUserWidget* UserWidgetCDO = PreviewableWidgetVariant->AsUserWidgetCDO())
		{
			UClass* WidgetClass = UserWidgetCDO->GetClass();

			// Get all of the functions we need to display (done ahead of time so we can sort them)
			for (TFieldIterator<UFunction> FunctionIter(WidgetClass, EFieldIteratorFlags::IncludeSuper); FunctionIter; ++FunctionIter)
			{
				UFunction* TestFunction = *FunctionIter;
				if (CanDisplayAndCallFunction(TestFunction))
				{
					bool bAllowFunction = true;
					if (UClass* TestFunctionOwnerClass = TestFunction->GetOwnerClass())
					{
						if (UBlueprint* Blueprint = Cast<UBlueprint>(TestFunctionOwnerClass->ClassGeneratedBy))
						{
							if (FBlueprintEditorUtils::IsEditorUtilityBlueprint(Blueprint))
							{
								// Skip Blutilities if disabled via project settings
								bAllowFunction = !bDisallowEditorUtilityBlueprintFunctions;
							}
						}
					}

					if (bAllowFunction)
					{
						const FName FunctionName = TestFunction->GetFName();
						if (!OutCallInEditorFunctions.FindByPredicate([&FunctionName](const UFunction* Func) { return Func->GetFName() == FunctionName; }))
						{
							OutCallInEditorFunctions.Add(*FunctionIter);
						}
					}
				}
			}
		}

		if (!OutCallInEditorFunctions.IsEmpty())
		{
			// Sort the functions by category and then by DisplayPriority meta tag, and then by name
			OutCallInEditorFunctions.Sort([](const UFunction& A, const UFunction& B)
			{
				const int32 CategorySort = A.GetMetaData(FBlueprintMetadata::MD_FunctionCategory).Compare(B.GetMetaData(FBlueprintMetadata::MD_FunctionCategory));
				if (CategorySort != 0)
				{
					return (CategorySort <= 0);
				}
				else
				{
					FString DisplayPriorityAStr = A.GetMetaData(NAME_DisplayPriority);
					int32 DisplayPriorityA = (DisplayPriorityAStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityAStr));
					if (DisplayPriorityA == 0 && !FCString::IsNumeric(*DisplayPriorityAStr))
					{
						DisplayPriorityA = MAX_int32;
					}

					FString DisplayPriorityBStr = B.GetMetaData(NAME_DisplayPriority);
					int32 DisplayPriorityB = (DisplayPriorityBStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityBStr));
					if (DisplayPriorityB == 0 && !FCString::IsNumeric(*DisplayPriorityBStr))
					{
						DisplayPriorityB = MAX_int32;
					}

					return (DisplayPriorityA == DisplayPriorityB) ? (A.GetName() <= B.GetName()) : (DisplayPriorityA <= DisplayPriorityB);
				}
			});
		}

		// Return false if empty
		return !OutCallInEditorFunctions.IsEmpty();
	}

	// Most of this is from FObjectDetails::AddCallInEditorMethods
	void FPreviewableWidgetCustomization::AddCallInEditorFunctions(
		IDetailChildrenBuilder& ChildBuilder,
		const TArrayView<UFunction*>& InCallInEditorFunctions)
	{
		if (InCallInEditorFunctions.IsEmpty())
		{
			return;
		}

		UUserWidget* WidgetInstance = GetWidgetInstance();
		if (!WidgetInstance)
		{
			return;
		}

		WeakWidgetInstance = WidgetInstance;

		struct FCategoryEntry
		{
			FName CategoryName;
			FName RowTag;
			TSharedPtr<SWrapBox> WrapBox;
			FTextBuilder FunctionSearchText;

			FCategoryEntry(FName InCategoryName)
				: CategoryName(InCategoryName)
			{
				WrapBox = SNew(SWrapBox)
					// Setting the preferred size here (despite using UseAllottedSize) is a workaround for an issue
					// when contained in a scroll box: prior to the first tick, the wrap box will use preferred size
					// instead of allotted, and if preferred size is set small, it will cause the box to wrap a lot and
					// request too much space from the scroll box. On next tick, SWrapBox is updated but the scroll box
					// does not realize that it needs to show more elements, until it is scrolled.
					// Setting a large value here means that the SWrapBox will request too little space prior to tick,
					// which will cause the scroll box to virtualize more elements at the start, but this is less broken.
					.PreferredSize(2000)
					.UseAllottedSize(true);
			}
		};

		// Build up a set of functions for each category, accumulating search text and buttons in a wrap box
		FName ActiveCategory;
		TArray<FCategoryEntry, TInlineAllocator<8>> CategoryList;
		for (UFunction* Function : InCallInEditorFunctions)
		{
			if (!Function)
			{
				continue;
			}

			FName FunctionCategoryName(NAME_Default);
			if (Function->HasMetaData(FBlueprintMetadata::MD_FunctionCategory))
			{
				FunctionCategoryName = FName(*Function->GetMetaData(FBlueprintMetadata::MD_FunctionCategory));
			}

			if (FunctionCategoryName != ActiveCategory)
			{
				ActiveCategory = FunctionCategoryName;
				CategoryList.Emplace(FunctionCategoryName);
			}

			FCategoryEntry& CategoryEntry = CategoryList.Last();

			const FText ButtonCaption = ObjectTools::GetUserFacingFunctionName(Function);
			FText FunctionTooltip = Function->GetToolTipText();
			if (FunctionTooltip.IsEmpty())
			{
				FunctionTooltip = ButtonCaption;
			}

			TWeakObjectPtr<UFunction> WeakFunctionPtr(Function);
			CategoryEntry.WrapBox->AddSlot()
			.Padding(0.0f, 0.0f, 5.0f, 3.0f)
			[
				SNew(SButton)
				.Text(ButtonCaption)
				.OnClicked(FOnClicked::CreateSP(this, &FPreviewableWidgetCustomization::OnExecuteCallInEditorFunction, WeakFunctionPtr))
				.IsEnabled(this, &FPreviewableWidgetCustomization::CanExecuteCallInEditorFunction, WeakFunctionPtr)
				.ToolTipText(FunctionTooltip.IsEmptyOrWhitespace() ? LOCTEXT("CallInEditorTooltip", "Call an event on the selected object(s)") : FunctionTooltip)
			];

			CategoryEntry.RowTag = Function->GetFName();
			CategoryEntry.FunctionSearchText.AppendLine(ButtonCaption);
			CategoryEntry.FunctionSearchText.AppendLine(FunctionTooltip);

			if (ButtonCaption.ToString() != Function->GetName())
			{
				CategoryEntry.FunctionSearchText.AppendLine(FText::FromString(Function->GetName()));
			}
		}

		IDetailGroup& FunctionsGroup = ChildBuilder.AddGroup("Functions", LOCTEXT("FunctionsGroupName", "Functions"));
		FunctionsGroup.ToggleExpansion(true);
		FunctionsGroup.SetToolTip(LOCTEXT("FunctionsGroupToolTip", "CallInEditor functions within the referenced widget."));

		TMap<FName, IDetailGroup*> Groups;

		// Now edit the categories, adding the button strips to the details panel
		for (FCategoryEntry& CategoryEntry : CategoryList)
		{
			IDetailGroup* Group = nullptr;
			if (CategoryEntry.CategoryName == NAME_Default)
			{
				Group = &FunctionsGroup;
			}
			else if (IDetailGroup** ExistingGroup = Groups.Find(CategoryEntry.CategoryName))
			{
				Group = *ExistingGroup;
			}
			else
			{
				Group = Groups.Emplace(
					CategoryEntry.CategoryName,
					&FunctionsGroup.AddGroup(
						CategoryEntry.CategoryName,
						FText::FromName(CategoryEntry.CategoryName)));
			}

			Group->AddWidgetRow()
			.FilterString(CategoryEntry.FunctionSearchText.ToText())
			.ShouldAutoExpand(true)
			.RowTag(CategoryEntry.RowTag)
			[
				CategoryEntry.WrapBox.ToSharedRef()
			];
		}
	}

	FReply FPreviewableWidgetCustomization::OnExecuteCallInEditorFunction(TWeakObjectPtr<UFunction> InWeakFunction)
	{
		TArray<TWeakObjectPtr<UObject>> WeakExecutionObjects = GetFunctionCallExecutionContext(InWeakFunction);
		if (UFunction* Function = InWeakFunction.Get())
		{
			FScopedTransaction Transaction(LOCTEXT("ExecuteCallInEditorMethod", "Call In Editor Action"));
			if (!WeakExecutionObjects.IsEmpty())
			{
				TStrongObjectPtr<UFunction> StrongFunction(Function);
	
				FEditorScriptExecutionGuard ScriptGuard;
				for (const TWeakObjectPtr<UObject>& WeakExecutionObject : WeakExecutionObjects)
				{
					if (UObject* ExecutionObject = WeakExecutionObject.Get())
					{
						ensure(Function->ParmsSize == 0);
						TStrongObjectPtr<UObject> StrongExecutionObject(ExecutionObject); // Prevent GC during call
						ExecutionObject->ProcessEvent(Function, nullptr);
					}
				}
			}
		}

		return FReply::Handled();
	}
	
	TArray<TWeakObjectPtr<UObject>> FPreviewableWidgetCustomization::GetFunctionCallExecutionContext(TWeakObjectPtr<UFunction> InWeakFunction) const
	{
		return { GetWidgetInstance() };
	}

	bool FPreviewableWidgetCustomization::CanExecuteCallInEditorFunction(TWeakObjectPtr<UFunction> InWeakFunction) const
	{
		return InWeakFunction.Get() && WeakWidgetInstance.IsValid();
	}

	FPreviewableWidgetVariant* FPreviewableWidgetCustomization::GetPreviewableWidgetVariant() const
	{
		check(PreviewVariantHandle.IsValid() && PreviewVariantHandle->IsValidHandle());

		void* StructPtr = nullptr;
		PreviewVariantHandle->GetValueData(StructPtr);
		FPreviewableWidgetVariant* PreviewableWidgetVariant = reinterpret_cast<FPreviewableWidgetVariant*>(StructPtr);

		return PreviewableWidgetVariant;
	}

	UUserWidget* FPreviewableWidgetCustomization::GetWidgetInstance() const
	{
		if (const UWidgetPreview* OwningPreview = WeakOwningPreview.Get())
		{
			// @todo: return instance of THIS widget, not the root
			if (UUserWidget* WidgetInstance = OwningPreview->GetWidgetInstance())
			{
				return WidgetInstance;
			}
		}

		return nullptr;
	}

	void FPreviewableWidgetCustomization::OnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType)
	{
		if (InChangeType != EWidgetPreviewWidgetChangeType::Resized)
		{
			PropertyUtilities->RequestForceRefresh();
		}
	}
}

#undef LOCTEXT_NAMESPACE
