// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Input/Reply.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WidgetPreview.h"

class FPropertyEditorModule;
class IDetailLayoutBuilder;
class IPropertyUtilities;
class SComboButton;
class SWidget;
class UFunction;
class UUserWidget;

namespace UE::UMGWidgetPreview::Private
{
	class FPreviewableWidgetCustomization
		: public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ Begin IPropertyTypeCustomization
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		//~ End IPropertyTypeCustomization

	private:
		bool GetCallInEditorFunctions(TArray<UFunction*>& OutCallInEditorFunctions) const;
		void AddCallInEditorFunctions(IDetailChildrenBuilder& ChildBuilder, const TArrayView<UFunction*>& InCallInEditorFunctions);
		
		TArray<TWeakObjectPtr<UObject>> GetFunctionCallExecutionContext(TWeakObjectPtr<UFunction> InWeakFunction) const;

		// Executes the specified method on the widget instance
		FReply OnExecuteCallInEditorFunction(TWeakObjectPtr<UFunction> InWeakFunction);

		// Checks if the function can be called - ie. that it has an object to call with.
		bool CanExecuteCallInEditorFunction(TWeakObjectPtr<UFunction> InWeakFunction) const;

		FPreviewableWidgetVariant* GetPreviewableWidgetVariant() const;
		UUserWidget* GetWidgetInstance() const;

		void OnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType);

	private:
		TSharedPtr<IPropertyUtilities> PropertyUtilities;
		TWeakObjectPtr<UWidgetPreview> WeakOwningPreview;
		TSharedPtr<IPropertyHandle> PreviewVariantHandle;
		TSharedPtr<IPropertyHandle> ObjectPathHandle;
		TWeakObjectPtr<UUserWidget> WeakWidgetInstance;
	};
}
