// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPreviewCustomization.h"

#include "Blueprint/UserWidget.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "WidgetPreviewCustomization"

namespace UE::UMGWidgetPreview::Private
{
	const FName FWidgetPreviewCustomization::WidgetTypePropertyName = "WidgetType";
	const FName FWidgetPreviewCustomization::SlotWidgetTypesPropertyName = "SlotWidgetTypes";
	const FName FWidgetPreviewCustomization::WidgetInstancePropertyName = "WidgetInstance";
	const FName FWidgetPreviewCustomization::OverriddenSizePropertyName = "OverriddenWidgetSize";

	TSharedRef<IDetailCustomization> FWidgetPreviewCustomization::MakeInstance()
	{
		TSharedRef<FWidgetPreviewCustomization> Customization = MakeShared<FWidgetPreviewCustomization>();
		return Customization;
	}

	void FWidgetPreviewCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		TArray<TWeakObjectPtr<UWidgetPreview>> CustomizedWidgetPreviews = DetailBuilder.GetObjectsOfTypeBeingCustomized<UWidgetPreview>();

		TSharedRef<IPropertyHandle> WidgetTypePropertyHandle = DetailBuilder.GetProperty(WidgetTypePropertyName);
		TSharedRef<IPropertyHandle> SlotWidgetTypesPropertyHandle = DetailBuilder.GetProperty(SlotWidgetTypesPropertyName);
		TSharedRef<IPropertyHandle> WidgetInstancePropertyHandle = DetailBuilder.GetProperty(WidgetInstancePropertyName);
		TSharedRef<IPropertyHandle> OverriddenSizePropertyHandle = DetailBuilder.GetProperty(OverriddenSizePropertyName);

		static const FName WidgetCategoryName = "Widget";

		for (TWeakObjectPtr<UWidgetPreview>& WeakWidgetPreview : CustomizedWidgetPreviews)
		{
			if (UWidgetPreview* WidgetPreview = WeakWidgetPreview.Get())
			{
				IDetailCategoryBuilder& WidgetCategory = DetailBuilder.EditCategory(WidgetCategoryName);
				WidgetCategory.AddProperty(WidgetTypePropertyHandle);
				WidgetCategory.AddProperty(OverriddenSizePropertyHandle);

				// Widget Instance
				{
					FAddPropertyParams AddPropertyParams;

					WidgetCategory.AddExternalObjects(
						{ WidgetPreview->GetWidgetInstance() },
						EPropertyLocation::Default,
						AddPropertyParams)
					->DisplayName(LOCTEXT("WidgetInstanceHeader", "Widget Properties"))
					.ShouldAutoExpand(true);
				}

				// Slot Widgets
				if (SlotWidgetTypesPropertyHandle->IsValidHandle())
				{
					TAttribute<bool> SlotWidgetTypesEditCondition = TAttribute<bool>::CreateLambda(
						[WeakWidgetPreview]()
						{
							if (UWidgetPreview* StrongWidgetPreview = WeakWidgetPreview.Get())
							{
								return !StrongWidgetPreview->GetWidgetSlotNames().IsEmpty();
							}

							return false;
						});

					WidgetCategory.AddProperty(SlotWidgetTypesPropertyHandle)
					.EditCondition(SlotWidgetTypesEditCondition, { });
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
