// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WidgetPreview.h"

class FPropertyEditorModule;
class IDetailLayoutBuilder;
class IPropertyUtilities;
class SComboButton;
class SWidget;
class UFunction;
class UUserWidget;

namespace UE::UMGWidgetPreview::Private
{
	class FWidgetPreviewCustomization
		: public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance();

		//~ Begin IDetailCustomization
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		//~ End IDetailCustomization

	private:
		static const FName WidgetTypePropertyName;
        static const FName SlotWidgetTypesPropertyName;
        static const FName WidgetInstancePropertyName;
        static const FName OverriddenSizePropertyName;
	};
}
