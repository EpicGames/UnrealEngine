// Copyright Epic Games, Inc. All Rights Reserved.

#include "UMGWidgetPreviewModule.h"

#include "Blueprint/UserWidget.h"
#include "ContentBrowserMenuContexts.h"
#include "Customizations/PreviewableWidgetCustomization.h"
#include "Customizations/WidgetPreviewCustomization.h"
#include "Editor.h"
#include "Framework/Commands/UIAction.h"
#include "Logging/LogMacros.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenuMisc.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectHash.h"
#include "WidgetBlueprint.h"
#include "WidgetPreview.h"
#include "WidgetPreviewCommands.h"
#include "WidgetPreviewEditor.h"
#include "WidgetPreviewLog.h"
#include "WidgetPreviewStyle.h"

#define LOCTEXT_NAMESPACE "UMGWidgetPreviewModule"

DEFINE_LOG_CATEGORY(LogWidgetPreview);

namespace UE::UMGWidgetPreview::Private
{
	static const FLazyName PreviewWidgetVariantName("PreviewableWidgetVariant");
	static const FLazyName WidgetPreviewName("WidgetPreview");

	void FUMGWidgetPreviewModule::StartupModule()
	{
		FWidgetPreviewCommands::Register();

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = false;
		InitOptions.bShowPages = false;
		InitOptions.bShowInLogWindow = false;
		InitOptions.bAllowClear = true;
		MessageLogModule.RegisterLogListing(MessageLogName, LOCTEXT("WidgetPreviewLog", "Widget Preview Log"), InitOptions);

		// Menus need to be registered in a callback to make sure the system is ready for them.
		UToolMenus::RegisterStartupCallback(
			FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FUMGWidgetPreviewModule::RegisterMenus));

		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		{
			PropertyModule.RegisterCustomPropertyTypeLayout(
				PreviewWidgetVariantName,
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPreviewableWidgetCustomization::MakeInstance));

			PropertyModule.RegisterCustomClassLayout(
				WidgetPreviewName,
				FOnGetDetailCustomizationInstance::CreateStatic(&FWidgetPreviewCustomization::MakeInstance));
		}
	}

	void FUMGWidgetPreviewModule::ShutdownModule()
	{
		UToolMenus::UnRegisterStartupCallback(this);

		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->UnregisterCustomClassLayout(WidgetPreviewName);
			PropertyModule->UnregisterCustomPropertyTypeLayout(PreviewWidgetVariantName);
		}

		if (FMessageLogModule* MessageLogModule = FModuleManager::GetModulePtr<FMessageLogModule>("MessageLog"))
		{
			MessageLogModule->UnregisterLogListing(MessageLogName);
		}

		FWidgetPreviewCommands::Unregister();
	}

	IUMGWidgetPreviewModule::FOnRegisterTabs& FUMGWidgetPreviewModule::OnRegisterTabsForEditor()
	{
		return RegisterTabsForEditorDelegate;
	}

	void FUMGWidgetPreviewModule::RegisterMenus()
	{
		// Allows cleanup when module unloads.
		FToolMenuOwnerScoped OwnerScoped(this);

		// Extend the content browser context menu for widgets
		{
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.WidgetBlueprint");
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry("OpenWidgetPreviewEditor", FNewToolMenuSectionDelegate::CreateLambda(
				[](FToolMenuSection& Section)
				{
					// We'll need to get the target assets out of the context
					if (UContentBrowserAssetContextMenuContext* Context = Section.FindContext<UContentBrowserAssetContextMenuContext>())
					{
						// We are deliberately not using Context->GetSelectedObjects() here to avoid triggering a load from right-clicking
						// an asset in the content browser.
						if (UWidgetPreviewEditor::AreAssetsValidTargets(Context->SelectedAssets))
						{
							const TSharedPtr<class FUICommandList> CommandListToBind = MakeShared<FUICommandList>();
							CommandListToBind->MapAction(
								FWidgetPreviewCommands::Get().OpenEditor,
								FExecuteAction::CreateLambda(
									[WeakContext = TWeakObjectPtr<UContentBrowserAssetContextMenuContext>(Context)]()
									{
										check(GEditor);

										if (UContentBrowserAssetContextMenuContext* Context = WeakContext.Get())
										{
											// When we actually do want to open the editor, trigger the load to get the objects
											TArray<TObjectPtr<UObject>> ObjectsToEdit;
											ObjectsToEdit.Append(Context->LoadSelectedObjects<UObject>());

											// If we fail the ensure here, then there must be something that we're failing to check properly
											// in AreAssetsValidTargets that we would need to track down and check in the asset data.
											if (ensure(UWidgetPreviewEditor::AreObjectsValidTargets(ObjectsToEdit)))
											{
												for (const TObjectPtr<UObject>& ObjectToEdit : ObjectsToEdit)
												{
													if (const UWidgetBlueprint* AsWidgetBlueprint = Cast<UWidgetBlueprint>(ObjectToEdit))
													{
														UWidgetPreview* PreviewForWidget = UWidgetPreviewEditor::CreatePreviewForWidget(
															Cast<UUserWidget>(AsWidgetBlueprint->GeneratedClass->GetDefaultObject<UUserWidget>()));

														GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(PreviewForWidget);
													}
												}
											}
										}
									}),
								FCanExecuteAction::CreateWeakLambda(Context, [Context]() { return Context->bCanBeModified; }));

							const TAttribute<FText> ToolTipOverride = Context->bCanBeModified ? TAttribute<FText>() : LOCTEXT("ReadOnlyAssetWarning", "The selected asset(s) are read-only and cannot be edited.");
							Section.AddMenuEntryWithCommandList(
								FWidgetPreviewCommands::Get().OpenEditor,
								CommandListToBind,
								LOCTEXT("WidgetContextMenuPreviewLabel", "Preview"), // Just use "Preview" here, the context means it's already a "Widget" so we can omit the prefix
								ToolTipOverride,
								FSlateIcon(FWidgetPreviewStyle::Get().GetStyleSetName(), "WidgetPreview.OpenEditor"));
						}
					}
				}));
		}
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(UE::UMGWidgetPreview::Private::FUMGWidgetPreviewModule, UMGWidgetPreview)
