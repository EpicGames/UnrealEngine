// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UMGWidgetPreview/Public/IUMGWidgetPreviewModule.h"

namespace UE::UMGWidgetPreview::Private
{
	class FUMGWidgetPreviewModule
		: public IUMGWidgetPreviewModule
	{
	public:
		//~ Begin IModuleInterface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface

		//~ Begin IUMGWidgetPreviewModule
		virtual FOnRegisterTabs& OnRegisterTabsForEditor() override;
		//~ End IUMGWidgetPreviewModule

	private:
		void RegisterMenus();

	private:
		FOnRegisterTabs RegisterTabsForEditorDelegate;
	};
}
