// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPreview.h"

#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetTree.h"
#include "Engine/Engine.h"
#include "UObject/Package.h"
#include "WidgetBlueprint.h"
#include "WidgetPreviewLog.h"
#include "WidgetPreviewTypesPrivate.h"
#include "Widgets/SWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetPreview)

FPreviewableWidgetVariant::FPreviewableWidgetVariant(const TSubclassOf<UUserWidget>& InWidgetType)
	: ObjectPath(InWidgetType)
{
	UpdateCachedWidget();
}

FPreviewableWidgetVariant::FPreviewableWidgetVariant(const UWidgetPreview* InWidgetPreview)
	: ObjectPath(InWidgetPreview)
{
	UpdateCachedWidget();
}

void FPreviewableWidgetVariant::UpdateCachedWidget()
{
	UE::UMGWidgetPreview::Private::FWidgetTypeTuple PreviewUserWidgetTuple;

	const UUserWidget* PreviewUserWidgetCDO = nullptr;
	CachedWidgetPreview.Reset();

	if (UObject* ResolvedObject = ObjectPath.TryLoad())
    {
    	if (UWidgetPreview* WidgetPreview = Cast<UWidgetPreview>(ResolvedObject))
    	{
    		CachedWidgetPreview = WidgetPreview;

    		// If we have a widget preview, it may not necessarily have a valid referenced widget type to preview
    		if (const UUserWidget* WidgetPreviewWidgetCDO = WidgetPreview->GetWidgetCDO())
    		{
    			PreviewUserWidgetCDO = WidgetPreviewWidgetCDO;
    			PreviewUserWidgetTuple.Set(WidgetPreviewWidgetCDO);
    		}
    	}
		else if (const UWidgetBlueprint* AsBlueprint = Cast<UWidgetBlueprint>(ResolvedObject))
		{
			PreviewUserWidgetCDO = AsBlueprint->GeneratedClass->GetDefaultObject<UUserWidget>();
		}
		else if (const UClass* AsClass = Cast<UClass>(ResolvedObject))
		{
			if (const UUserWidget* UserWidgetCDO = AsClass->GetDefaultObject<UUserWidget>())
			{
				PreviewUserWidgetCDO = UserWidgetCDO;
			}
		}
    }

	CachedWidgetCDO = PreviewUserWidgetCDO;
}

const UUserWidget* FPreviewableWidgetVariant::AsUserWidgetCDO() const
{
	if (ObjectPath.IsNull())
	{
		return nullptr;
	}

	if (const UUserWidget* UserWidgetCDO = CachedWidgetCDO.Get())
	{
		return UserWidgetCDO;
	}

	if (UObject* ResolvedObject = ObjectPath.TryLoad())
	{
		if (const UClass* AsClass = Cast<UClass>(ResolvedObject))
		{
			if (UUserWidget* UserWidget = AsClass->GetDefaultObject<UUserWidget>())
			{
				UE_LOGF(LogWidgetPreview, Warning, "Tried to get the object as a UserWidget (CDO), but it wasn't cached. Ensure you have called Refresh() first.");
				return UserWidget;
			}
		}
	}

	return nullptr;
}

const UWidgetPreview* FPreviewableWidgetVariant::AsWidgetPreview() const
{
	if (ObjectPath.IsNull())
	{
		return nullptr;
	}

	if (UWidgetPreview* WidgetPreview = CachedWidgetPreview.Get())
	{
		return WidgetPreview;
	}

	if (UObject* ResolvedObject = ObjectPath.TryLoad())
	{
		if (UWidgetPreview* WidgetPreview = Cast<UWidgetPreview>(ResolvedObject))
		{
			UE_LOGF(LogWidgetPreview, Warning, "Tried to get the object as a WidgetPreview, but it wasn't cached. Ensure you have called Refresh() first.");
			return WidgetPreview;
		}
	}

	return nullptr;
}

UWidgetPreview::UWidgetPreview(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShouldOverrideWidgetSize(false)
	, bDefaultConstructViewmodels(false)
{
	OverriddenWidgetSize = FVector2D(100,100);
}

const TArray<FName>& UWidgetPreview::GetWidgetSlotNames() const
{
	return SlotNameCache;
}

UUserWidget* UWidgetPreview::GetOrCreateWidgetInstance(UWorld* InWorld, const bool bInForceRecreate)
{
	if (bInForceRecreate)
	{
		ClearWidgetInstance();
	}
	else if (WidgetInstance)
	{
		return WidgetInstance;
	}

	return CreateWidgetInstance(InWorld);
}

UUserWidget* UWidgetPreview::CreateWidgetInstance(UWorld* InWorld)
{
	if (!ensure(InWorld))
	{
		return nullptr;
	}

	TArray<const UUserWidget*> UnsupportedWidgets;
	if (!CanCallInitializedWithoutPlayerContext(true, UnsupportedWidgets))
	{
		// No need to log, this is an expected outcome
		return nullptr;
	}

	if (const UUserWidget* Widget = GetWidgetCDO())
	{
		auto MakeWidget = [InWorld](UClass* InClass) -> UUserWidget*
		{
			ensureAlways(!InClass->GetName().StartsWith(TEXT("REINST_")));

			UUserWidget* NewWidget = NewObject<UUserWidget>(InWorld, InClass);
			NewWidget->ClearFlags(RF_Transactional);
			return NewWidget;
		};

		WidgetInstance = MakeWidget(Widget->GetClass());

		if (!WidgetType.ObjectPath.IsNull() && !SlotWidgetTypes.IsEmpty())
		{
			TArray<FName> ValidSlotNames;
			WidgetInstance->GetSlotNames(ValidSlotNames);

			for (TPair<FName, FPreviewableWidgetVariant>& SlotWidget : SlotWidgetTypes)
			{
				if (!SlotWidget.Value.ObjectPath.IsNull()
					&& ValidSlotNames.Contains(SlotWidget.Key))
				{
					WidgetInstance->SetContentForSlot(SlotWidget.Key, MakeWidget(SlotWidget.Value.AsUserWidgetCDO()->GetClass()));
				}
			}
		}

		if (ULocalPlayer* LocalPlayer = InWorld->GetFirstLocalPlayerFromController())
		{
			WidgetInstance->SetPlayerContext(LocalPlayer);
		}

		SlateWidgetInstance = WidgetInstance->TakeWidget();

		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Reinstanced);

		return WidgetInstance;
	}

	return nullptr;
}

UUserWidget* UWidgetPreview::GetWidgetInstance() const
{
	return WidgetInstance;
}

TSharedPtr<SWidget> UWidgetPreview::GetSlateWidgetInstance() const
{
	if (SlateWidgetInstance.IsValid())
	{
		return SlateWidgetInstance;
	}

	if (UUserWidget* Instance = GetWidgetInstance())
	{
		return Instance->TakeWidget();
	}

	return nullptr;
}

const UUserWidget* UWidgetPreview::GetWidgetCDO() const
{
	// If the LayoutWidget is in use, return it (as the root widget).
	if (!WidgetType.ObjectPath.IsNull())
	{
		const UUserWidget* LayoutWidgetCDO = WidgetType.AsUserWidgetCDO();
		if (const INamedSlotInterface* WidgetWithSlots = Cast<INamedSlotInterface>(LayoutWidgetCDO))
		{
			TArray<FName> SlotNames;
			WidgetWithSlots->GetSlotNames(SlotNames);
			if (!SlotNames.IsEmpty())
			{
				return LayoutWidgetCDO;
			}
		}
	}

	if (!WidgetType.ObjectPath.IsNull())
	{
		return WidgetType.AsUserWidgetCDO();
	}

	return nullptr;
}

const UUserWidget* UWidgetPreview::GetWidgetCDOForSlot(const FName InSlotName) const
{
	if (const FPreviewableWidgetVariant* WidgetInSlot = SlotWidgetTypes.Find(InSlotName))
	{
		if (!WidgetInSlot->ObjectPath.IsNull())
		{
			return WidgetInSlot->AsUserWidgetCDO();
		}

		UE_LOGF(LogTemp, Warning, "Slot %ls has invalid widget.", *InSlotName.ToString());
	}

	return nullptr;
}

void UWidgetPreview::BeginDestroy()
{
	UObject::BeginDestroy();

	CleanupReferences();
}

bool UWidgetPreview::CanCallInitializedWithoutPlayerContext(const bool bInRecursive, TArray<const UUserWidget*>& OutFailedWidgets)
{
	bool bResult = true;

	if (!WidgetType.ObjectPath.IsNull())
	{
		const UUserWidget* WidgetCDO = WidgetType.AsUserWidgetCDO();
		bResult = bResult && CanCallInitializedWithoutPlayerContextOnWidget(WidgetCDO, bInRecursive, OutFailedWidgets);

		for (const TPair<FName, FPreviewableWidgetVariant>& SlotWidget : SlotWidgetTypes)
		{
			if (!SlotWidget.Value.ObjectPath.IsNull())
			{
				const UUserWidget* SlotWidgetCDO = SlotWidget.Value.AsUserWidgetCDO();
				bResult = bResult && CanCallInitializedWithoutPlayerContextOnWidget(SlotWidgetCDO, bInRecursive, OutFailedWidgets);
			}
		}
	}

	// In case there are no widgets to display, we want to return true
	return bResult;
}

bool UWidgetPreview::CanCallInitializedWithoutPlayerContextOnWidget(
	const UUserWidget* InUserWidget,
	const bool bInRecursive,
	TArray<const UUserWidget*>& OutFailedWidgets)
{
	TFunction<bool(const UWidget* InWidgetGeneratedClass)> CanCallInitializedWithoutPlayerContextInternal;
	CanCallInitializedWithoutPlayerContextInternal = [CanCallInitializedWithoutPlayerContextInternal, bInRecursive, &OutFailedWidgets](const UWidget* InWidget)
	{
		// In case there are no widgets to display, we want to return true
		bool bResultInternal = true;

		if (const UUserWidget* AsUserWidget = Cast<UUserWidget>(InWidget))
		{
			UE::UMGWidgetPreview::Private::FWidgetTypeTuple WidgetTuple(AsUserWidget);
			if (WidgetTuple.BlueprintGeneratedClass)
			{
				bResultInternal = WidgetTuple.BlueprintGeneratedClass->bCanCallInitializedWithoutPlayerContext;
				if (!bResultInternal)
				{
					OutFailedWidgets.Emplace(WidgetTuple.ClassDefaultObject);
				}
			}

			if (bInRecursive)
			{
				if (const INamedSlotInterface* WidgetWithSlots = Cast<INamedSlotInterface>(AsUserWidget))
				{
					TArray<FName> SlotNames;
					WidgetWithSlots->GetSlotNames(SlotNames);
					if (!SlotNames.IsEmpty())
					{
						for (const FName SlotName : SlotNames)
						{
							if (const UWidget* SlotWidget = Cast<UWidget>(WidgetWithSlots->GetContentForSlot(SlotName)))
							{
								bResultInternal = bResultInternal && CanCallInitializedWithoutPlayerContextInternal(SlotWidget);
							}
						}
					}
				}
			}
		}

		return bResultInternal;
	};

	return CanCallInitializedWithoutPlayerContextInternal(InUserWidget);
}

const FPreviewableWidgetVariant& UWidgetPreview::GetWidgetType() const
{
	return WidgetType;
}

void UWidgetPreview::SetWidgetType(const FPreviewableWidgetVariant& InWidget)
{
	if (WidgetType != InWidget)
	{
		WidgetType = InWidget;
		ClearWidgetInstance();

		// Check for self-reference
		if (WidgetType.ObjectPath == this)
		{
			UE_LOGF(LogWidgetPreview, Warning, "Tried to reference self as the widget type. This is not allowed.");
			WidgetType.ObjectPath.Reset();
		}

		UpdateWidgets();

		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Assignment);
	}
}

const TMap<FName, FPreviewableWidgetVariant>& UWidgetPreview::GetSlotWidgetTypes() const
{
	return SlotWidgetTypes;
}

void UWidgetPreview::SetSlotWidgetTypes(const TMap<FName, FPreviewableWidgetVariant>& InWidgets)
{
	if (!SlotWidgetTypes.OrderIndependentCompareEqual(InWidgets))
	{
		SlotWidgetTypes = InWidgets;
		ClearWidgetInstance();
		UpdateWidgets();

		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Assignment);
	}
}

const bool UWidgetPreview::GetbShouldOverrideWidgetSize() const
{
	return bShouldOverrideWidgetSize;
}

void UWidgetPreview::SetbShouldOverrideWidgetSize(bool InOverride)
{
	if (bShouldOverrideWidgetSize != InOverride)
	{
		bShouldOverrideWidgetSize = InOverride;
		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Resized);
	}
}

const FVector2D UWidgetPreview::GetOverriddenWidgetSize() const
{
	return OverriddenWidgetSize;
}

void UWidgetPreview::SetOverriddenWidgetSize(FVector2D InWidgetSize)
{
	if (OverriddenWidgetSize != InWidgetSize)
	{
		OverriddenWidgetSize = InWidgetSize;
		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Resized);
	}
	
}

bool UWidgetPreview::ShouldDefaultConstructViewmodels() const
{
	return bDefaultConstructViewmodels;
}

void UWidgetPreview::SetDefaultConstructViewmodels(bool bInDefaultConstruct)
{
	bDefaultConstructViewmodels = bInDefaultConstruct;
}

void UWidgetPreview::PostLoad()
{
	UObject::PostLoad();

	UpdateWidgets();
}

void UWidgetPreview::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWidgetPreview, WidgetType)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UWidgetPreview, SlotWidgetTypes)
		|| PropertyName.IsNone()) // None can be an Undo operation
	{
		ClearWidgetInstance();

		// Check for self-reference and disallow
		if (WidgetType.ObjectPath == this)
		{
			UE_LOGF(LogWidgetPreview, Warning, "Tried to reference self as the widget type. This is not allowed.");
			WidgetType.ObjectPath.Reset();
		}

		UpdateWidgets();
		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Assignment);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWidgetPreview, bShouldOverrideWidgetSize)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWidgetPreview, OverriddenWidgetSize))
	{
		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Resized);
	}
}

// Due to compilation, etc.
void UWidgetPreview::OnWidgetBlueprintChanged(UBlueprint* InBlueprint)
{
	ClearWidgetInstance();
	UpdateWidgets();
	OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Structure);
}

void UWidgetPreview::ClearWidgetInstance()
{
	if (WidgetInstance)
	{
		if (SlateWidgetInstance.IsValid())
		{
			SlateWidgetInstance.Reset();
		}

		OnWidgetChanged().Broadcast(EWidgetPreviewWidgetChangeType::Destroyed);

		WidgetInstance->OnNativeDestruct.RemoveAll(this);
		WidgetInstance->MarkAsGarbage();
		WidgetInstance = nullptr;

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
}

void UWidgetPreview::UpdateWidgets()
{
	auto AddOnChanged = [this](const UUserWidget* InUserWidgetCDO)
	{
		const UE::UMGWidgetPreview::Private::FWidgetTypeTuple WidgetTuple(InUserWidgetCDO);
		if (UWidgetBlueprint* BP = WidgetTuple.Blueprint)
		{
			BP->OnChanged().AddUObject(this, &UWidgetPreview::OnWidgetBlueprintChanged);
		}
	};

	CleanupReferences();

	WidgetType.UpdateCachedWidget();
	if (!WidgetType.ObjectPath.IsNull())
	{
		SlotNameCache.Reset();
		if (const UUserWidget* AsUserWidget = WidgetType.AsUserWidgetCDO())
		{
			WidgetReferenceCache.Emplace(MakeWeakObjectPtr(AsUserWidget));
			AddOnChanged(AsUserWidget);

			if (const INamedSlotInterface* WidgetWithSlots = Cast<INamedSlotInterface>(AsUserWidget))
			{
				TArray<FName> SlotNames;
				WidgetWithSlots->GetSlotNames(SlotNames);

				SlotNameCache = SlotNames;
			}
		}

		for (TPair<FName, FPreviewableWidgetVariant>& SlotWidget : SlotWidgetTypes)
		{
			SlotWidget.Value.UpdateCachedWidget();
			if (!SlotWidget.Value.ObjectPath.IsNull())
			{
				if (const UUserWidget* AsUserWidget = WidgetType.AsUserWidgetCDO())
				{
					WidgetReferenceCache.Emplace(MakeWeakObjectPtr(AsUserWidget));
					AddOnChanged(AsUserWidget);
				}
			}
		}
	}
}

void UWidgetPreview::CleanupReferences()
{
	// Clear previous references, required due to how Blueprints are handled when changed
	TArray<TWeakObjectPtr<const UUserWidget>> WidgetsToDeinitialize = WidgetReferenceCache;
	for (TWeakObjectPtr<const UUserWidget>& WeakUserWidget : WidgetsToDeinitialize)
	{
		if (const UUserWidget* UserWidget = WeakUserWidget.Get())
		{
			const UE::UMGWidgetPreview::Private::FWidgetTypeTuple WidgetTuple(UserWidget);
			if (UWidgetBlueprint* BP = WidgetTuple.Blueprint)
			{
				BP->OnChanged().RemoveAll(this);
			}
		}
	}
	WidgetReferenceCache.Reset();
}

TArray<FName> UWidgetPreview::GetAvailableWidgetSlotNames()
{
	const TArray<FName> AllSlotNames = GetWidgetSlotNames();

	TArray<FName> UsedSlotNamesArray;
	SlotWidgetTypes.GenerateKeyArray(UsedSlotNamesArray);

	const TSet<FName> UsedSlotNames(UsedSlotNamesArray);

	return TSet<FName>(AllSlotNames)
		.Difference(UsedSlotNames)
		.Array();
}
