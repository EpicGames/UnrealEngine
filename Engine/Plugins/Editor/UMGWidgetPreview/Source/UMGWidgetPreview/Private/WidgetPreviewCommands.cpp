// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPreviewCommands.h"

#include "InputCoreTypes.h"
#include "WidgetPreviewStyle.h"

#define LOCTEXT_NAMESPACE "FWidgetPreviewCommands"

namespace UE::UMGWidgetPreview::Private
{
	FWidgetPreviewCommands::FWidgetPreviewCommands()
		: TCommands<FWidgetPreviewCommands>("WidgetPreviewEditor",
			LOCTEXT("ContextDescription", "Widget Preview Editor"),
			NAME_None,
			FWidgetPreviewStyle::Get().GetStyleSetName())
	{
	}

	void FWidgetPreviewCommands::RegisterCommands()
	{
		UI_COMMAND(OpenEditor, "Widget Preview", "Opens the Widget Preview window.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ResetPreview, "Reset", "Resets the current Widget Preview.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::R));
	}
}

#undef LOCTEXT_NAMESPACE
