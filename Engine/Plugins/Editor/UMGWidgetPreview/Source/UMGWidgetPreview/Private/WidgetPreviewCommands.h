// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::UMGWidgetPreview::Private
{
	class FWidgetPreviewCommands
		: public TCommands<FWidgetPreviewCommands>
	{
	public:
		FWidgetPreviewCommands();

		//~ Begin TCommands
		virtual void RegisterCommands() override;
		//~ End TCommands

	public:
		TSharedPtr<FUICommandInfo> OpenEditor;

		/** Resets the state of the given preview. */
		TSharedPtr<FUICommandInfo> ResetPreview;
	};
}
