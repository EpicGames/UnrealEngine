// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPreviewEditor.h"

#include "Algo/AllOf.h"
#include "AssetToolsModule.h"
#include "Blueprint/UserWidget.h"
#include "PackageTools.h"
#include "WidgetBlueprint.h"
#include "WidgetPreview.h"
#include "WidgetPreviewFactory.h"
#include "WidgetPreviewToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetPreviewEditor)

void UWidgetPreviewEditor::Initialize(const TObjectPtr<UWidgetPreview>& InWidgetPreview)
{
	WidgetPreview = InWidgetPreview;

	UAssetEditor::Initialize();
}

void UWidgetPreviewEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	InObjectsToEdit.Emplace(WidgetPreview);
}

TSharedPtr<FBaseAssetToolkit> UWidgetPreviewEditor::CreateToolkit()
{
	return MakeShared<UE::UMGWidgetPreview::Private::FWidgetPreviewToolkit>(this);
}

void UWidgetPreviewEditor::FocusWindow(UObject* ObjectToFocusOn)
{
	if (ToolkitInstance)
	{
		ToolkitInstance->FocusWindow(ObjectToFocusOn);
	}
}

UWidgetPreview* UWidgetPreviewEditor::GetObjectToEdit() const
{
	return WidgetPreview;
}

bool UWidgetPreviewEditor::AreObjectsValidTargets(const TArray<UObject*>& InObjects)
{
	if (InObjects.IsEmpty())
	{
		return false;
	}

	return Algo::AllOf(InObjects, [](const UObject* InObject)
	{
		return InObject->IsA<UWidgetBlueprint>();
	});
}

bool UWidgetPreviewEditor::AreAssetsValidTargets(const TArray<FAssetData>& InAssets)
{
	if (InAssets.IsEmpty())
	{
		return false;
	}

	return Algo::AllOf(InAssets, [](const FAssetData& InAsset)
	{
		return InAsset.IsInstanceOf<UWidgetBlueprint>();
	});
}

UWidgetPreview* UWidgetPreviewEditor::CreatePreviewForWidget(const UUserWidget* InUserWidget)
{
	// Create a new widget preview (@see: UEditorEngine::NewMap)
	UWidgetPreviewFactory* Factory = NewObject<UWidgetPreviewFactory>();

	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	FString WidgetAssetPath;
	FString WidgetAssetName;
	InUserWidget->GetPackage()->GetName().Split(TEXT("/"), &WidgetAssetPath, &WidgetAssetName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

	const FString DesiredAssetName = WidgetAssetName + TEXT("_Preview");
	const FString DesiredPackagePath = UPackageTools::SanitizePackageName(WidgetAssetPath + TEXT("/") + DesiredAssetName);

	FString DefaultSuffix;
	FString AssetName;
	FString PackageName;
	AssetToolsModule.Get().CreateUniqueAssetName(DesiredPackagePath, DefaultSuffix, PackageName, AssetName);

	EObjectFlags Flags = RF_Public | RF_Standalone;
	UWidgetPreview* NewWidgetPreview = CastChecked<UWidgetPreview>(
		Factory->FactoryCreateNew(
			UWidgetPreview::StaticClass(),
			GetTransientPackage(),
			*AssetName,
			Flags,
			nullptr,
			GWarn));

	NewWidgetPreview->SetWidgetType(FPreviewableWidgetVariant(InUserWidget->GetClass()));
	NewWidgetPreview->MarkPackageDirty();

	return NewWidgetPreview;
}
