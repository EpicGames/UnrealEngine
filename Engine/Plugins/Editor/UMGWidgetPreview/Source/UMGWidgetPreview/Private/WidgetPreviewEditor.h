// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"

#include "WidgetPreviewEditor.generated.h"

class FBaseAssetToolkit;
class UObject;
class UUserWidget;
class UWidget;
class UWidgetPreview;

/** Minimalistic container to spawn the toolkit. */
UCLASS(Transient)
class UWidgetPreviewEditor
	: public UAssetEditor
{
	GENERATED_BODY()

public:
	void Initialize(const TObjectPtr<UWidgetPreview>& InWidgetPreview);

	//~ Begin UAssetEditor
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;
	virtual void FocusWindow(UObject* ObjectToFocusOn) override;
	//~ End UAssetEditor

	UWidgetPreview* GetObjectToEdit() const;

	/** Checks that all of the objects are valid targets for a Widget Preview session. */
	static bool AreObjectsValidTargets(const TArray<UObject*>& InObjects);

	/**
	 * Checks that all of the assets are valid targets for an editor session. This
	 * is preferable over AreObjectsValidTargets when we have FAssetData because it
	 * allows us to avoid forcing a load of the underlying UObjects (for instance to
	 * avoid triggering a load when right clicking an asset in the content browser).
	 */
	static bool AreAssetsValidTargets(const TArray<FAssetData>& InAssets);

	/** Creates a new, unsaved Widget Preview asset for the given UUserWidget. */
	static UWidgetPreview* CreatePreviewForWidget(const UUserWidget* InUserWidget);

private:
	TObjectPtr<UWidgetPreview> WidgetPreview = nullptr;
};
