// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPreviewFactory.h"

#include "WidgetPreview.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetPreviewFactory)

UWidgetPreviewFactory::UWidgetPreviewFactory()
{
	SupportedClass = UWidgetPreview::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UWidgetPreviewFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(InClass == UWidgetPreview::StaticClass());

	UWidgetPreview* WidgetPreviewAsset = NewObject<UWidgetPreview>(InParent, InName, Flags);
	
	return WidgetPreviewAsset;
}
