// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "WidgetPreviewFactory.generated.h"

UCLASS(HideCategories = Object, CollapseCategories, MinimalAPI)
class UWidgetPreviewFactory : public UFactory
{
	GENERATED_BODY()

public:
	UWidgetPreviewFactory();
	
	//~ Begin UFactory
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ End UFactory
};
