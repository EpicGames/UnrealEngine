// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPreviewStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"

namespace UE::UMGWidgetPreview::Private
{
	FName FWidgetPreviewStyle::StyleName("WidgetPreview");

	FWidgetPreviewStyle& FWidgetPreviewStyle::Get()
	{
		static FWidgetPreviewStyle Instance;
		return Instance;
	}

	FWidgetPreviewStyle::FWidgetPreviewStyle()
		: FSlateStyleSet(StyleName)
	{
		const FVector2f Icon16(16.0f);
		const FVector2f Icon64(64.0f);

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
		check(Plugin.IsValid());

		SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

		Set("WidgetPreview.OpenEditor", new IMAGE_BRUSH_SVG("Icons/WidgetPreview_16", Icon16));
		Set("WidgetPreview.Reset", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Reset", Icon16));

		// Custom Class Icons
		Set("ClassIcon.WidgetPreview", new IMAGE_BRUSH_SVG(TEXT("Icons/WidgetPreview_16"), Icon16));
		Set("ClassThumbnail.WidgetPreview", new IMAGE_BRUSH_SVG(TEXT("Icons/WidgetPreview_64"), Icon64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FWidgetPreviewStyle::~FWidgetPreviewStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}
