// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::UMGWidgetPreview::Private
{
	class FWidgetPreviewStyle
		: public FSlateStyleSet
	{
	public:
		static FName StyleName;

		static FWidgetPreviewStyle& Get();

	private:
		FWidgetPreviewStyle();
		virtual ~FWidgetPreviewStyle() override;
	};
}
