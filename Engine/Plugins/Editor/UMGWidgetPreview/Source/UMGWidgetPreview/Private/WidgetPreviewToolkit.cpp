// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPreviewToolkit.h"

#include "DataValidationFixers.h"
#include "FileHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "MessageLogModule.h"
#include "ObjectEditorUtils.h"
#include "SAssetEditorViewport.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuMisc.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UMGWidgetPreview/Public/IUMGWidgetPreviewModule.h"
#include "WidgetPreview.h"
#include "WidgetPreviewCommands.h"
#include "WidgetPreviewEditor.h"
#include "WidgetPreviewLog.h"
#include "WidgetPreviewStyle.h"
#include "WidgetPreviewTypesPrivate.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SWidgetPreview.h"
#include "Widgets/SWidgetPreviewDetails.h"
#include "Widgets/SWidgetPreviewStatus.h"
#include "Widgets/SWidgetPreviewViewport.h"

#define LOCTEXT_NAMESPACE "WidgetPreviewToolkit"

namespace UE::UMGWidgetPreview::Private
{
	const FLazyName FWidgetPreviewToolkit::PreviewSceneSettingsTabID(TEXT("WidgetPreviewToolkit_PreviewScene"));
	const FLazyName FWidgetPreviewToolkit::MessageLogTabID(TEXT("WidgetPreviewToolkit_MessageLog"));

	EFixApplicability FWidgetPreviewabilityFixer::GetApplicability(int32 FixIndex) const
	{
		if (const UUserWidget* UserWidget = WeakUserWidget.Get())
		{
			const FWidgetTypeTuple WidgetTuple(UserWidget);
			if (WidgetTuple.BlueprintGeneratedClass)
			{
				return WidgetTuple.BlueprintGeneratedClass->bCanCallInitializedWithoutPlayerContext
					? EFixApplicability::Applied
					: EFixApplicability::CanBeApplied;
			}
		}

		return EFixApplicability::DidNotApply;
	}

	FFixResult FWidgetPreviewabilityFixer::ApplyFix(int32 FixIndex)
	{
		// @todo: apply recursively (named slots, etc.)
		if (const UUserWidget* UserWidget = WeakUserWidget.Get())
		{
			FWidgetTypeTuple WidgetTuple(UserWidget);
			if (UWidgetBlueprint* WidgetBlueprint = WidgetTuple.Blueprint)
			{
				FScopedTransaction Transaction(LOCTEXT("FixWidgetBlueprint", "Fix Widget Blueprint"));

				// Set flag
				{
					FObjectEditorUtils::SetPropertyValue(
						WidgetBlueprint,
						GET_MEMBER_NAME_CHECKED(UWidgetBlueprint, bCanCallInitializedWithoutPlayerContext),
						true);
				}

				// Compile
				{
					FKismetEditorUtilities::CompileBlueprint(WidgetBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);
					WidgetBlueprint->PostEditChange();
					WidgetBlueprint->MarkPackageDirty();
				}

				return FFixResult::Success();
			}
		}

		return FFixResult::Failure(LOCTEXT("FixWidgetBlueprint_Failure", "Failed to fix UserWidget."));
	}

	TSharedRef<FWidgetPreviewabilityFixer> FWidgetPreviewabilityFixer::Create(const UUserWidget* InUserWidget)
	{
		TSharedRef<FWidgetPreviewabilityFixer> Fixer = MakeShared<FWidgetPreviewabilityFixer>();
		Fixer->WeakUserWidget = InUserWidget;
		return Fixer;
	}

	FWidgetPreviewScene::FWidgetPreviewScene(const TSharedRef<FWidgetPreviewToolkit>& InPreviewToolkit)
		: WeakToolkit(InPreviewToolkit)
		, PreviewScene(MakeShared<FAdvancedPreviewScene>(
			FPreviewScene::ConstructionValues()
			.AllowAudioPlayback(true)
			.ShouldSimulatePhysics(true)
			.SetEditor(false)))
	{
		PreviewScene->SetFloorVisibility(false);
	}

	void FWidgetPreviewScene::Tick(float DeltaTime)
	{
		if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			if (!Toolkit->GetState()->CanTick()
				|| GEditor->bIsSimulatingInEditor
				|| GEditor->PlayWorld != nullptr)
			{
				return;
			}

			PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaTime);
		}
	}

	ETickableTickType FWidgetPreviewScene::GetTickableTickType() const
	{
		return ETickableTickType::Always;
	}

	TStatId FWidgetPreviewScene::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FWidgetPreviewScene, STATGROUP_Tickables);
	}

	UWorld* FWidgetPreviewScene::GetWorld() const
	{
		return PreviewScene->GetWorld();
	}

	TSharedRef<FAdvancedPreviewScene> FWidgetPreviewScene::GetPreviewScene() const
	{
		return PreviewScene.ToSharedRef();
	}

	FWidgetPreviewToolkitStateBase::FWidgetPreviewToolkitStateBase(const FName& Id)
		: Id(Id)
	{
	}

	FName FWidgetPreviewToolkitStateBase::GetId() const
	{
		return Id;
	}

	const TSharedPtr<FTokenizedMessage>& FWidgetPreviewToolkitStateBase::GetStatusMessage() const
	{
		return StatusMessage;
	}

	bool FWidgetPreviewToolkitStateBase::CanTick() const
	{
		return bCanTick;
	}

	bool FWidgetPreviewToolkitStateBase::ShouldOverlayStatusMessage() const
	{
		return bShouldOverlayMessage;
	}

	void FWidgetPreviewToolkitStateBase::OnEnter(const FWidgetPreviewToolkitStateBase* InFromState)
	{
		// Default, empty Implementation
	}

	void FWidgetPreviewToolkitStateBase::OnExit(const FWidgetPreviewToolkitStateBase* InToState)
	{
		// Default, empty Implementation
	}

	FWidgetPreviewToolkitPausedState::FWidgetPreviewToolkitPausedState(): FWidgetPreviewToolkitStateBase(TEXT("Paused"))
	{
		StatusMessage = FTokenizedMessage::Create(EMessageSeverity::Info, LOCTEXT("WidgetPreviewToolkitPausedState_Message", "The preview is currently paused."));
		bCanTick = false;
		bShouldOverlayMessage = true;
	}

	FWidgetPreviewToolkitBackgroundState::FWidgetPreviewToolkitBackgroundState()
	{
		Id = TEXT("Background");
		StatusMessage = FTokenizedMessage::Create(EMessageSeverity::Info, LOCTEXT("WidgetPreviewToolkitBackgroundState_Message", "The widget preview is paused while the window is in the background. Re-focus to unpause."));
	}

	FWidgetPreviewToolkitUnsupportedWidgetState::FWidgetPreviewToolkitUnsupportedWidgetState()
	{
		Id = TEXT("UnsupportedWidget");
		ResetStatusMessage();
	}

	void FWidgetPreviewToolkitUnsupportedWidgetState::SetUnsupportedWidgets(const TArray<const UUserWidget*>& InWidgets)
	{
		UnsupportedWidgets.Reset();

		Algo::Transform(
			InWidgets,
			UnsupportedWidgets,
			[](const UUserWidget* InWidget){
				return MakeWeakObjectPtr(InWidget);
			});

		// Reset message
		ResetStatusMessage();

		for (const TWeakObjectPtr<const UUserWidget>& WeakUnsupportedWidget : UnsupportedWidgets)
		{
			if (const UUserWidget* UnsupportedWidget = WeakUnsupportedWidget.Get())
			{
				const TSharedRef<FWidgetPreviewabilityFixer> WidgetFixer = FWidgetPreviewabilityFixer::Create(UnsupportedWidget);

				StatusMessage->AddToken(FAssetNameToken::Create(UnsupportedWidget->GetPackage()->GetName()));
				StatusMessage->AddToken(WidgetFixer->CreateToken(LOCTEXT("FixUnsupportedWidget", "Fix")));
			}
		}
	}

	void FWidgetPreviewToolkitUnsupportedWidgetState::ResetStatusMessage()
	{
		StatusMessage = FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("WidgetPreviewToolkitUnsupportedWidgetState_Message", "One or more referenced widgets isn't supported (\"Can Call Initialized Without Player Context\" might be disabled)."));
	}

	FWidgetPreviewToolkitRunningState::FWidgetPreviewToolkitRunningState(): FWidgetPreviewToolkitStateBase(TEXT("Running"))
	{
		StatusMessage = FTokenizedMessage::Create(EMessageSeverity::Info, LOCTEXT("WidgetPreviewToolkitRunningState_Message", "The preview is running!"));
		bCanTick = true;
		bShouldOverlayMessage = false;
	}

	FWidgetPreviewToolkit::FWidgetPreviewToolkit(UWidgetPreviewEditor* InOwningEditor)
		: FBaseAssetToolkit(InOwningEditor)
		, Preview(InOwningEditor->GetObjectToEdit())
	{
		StandaloneDefaultLayout = FTabManager::NewLayout("WidgetPreviewEditor_Layout_v1")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.85f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.85f)
						->AddTab(ViewportTabID, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
					->Split
					(
					FTabManager::NewStack()
						->SetSizeCoefficient(0.15f)
						->AddTab(MessageLogTabID, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(DetailsTabID, ETabState::OpenedTab)
						->AddTab(PreviewSceneSettingsTabID, ETabState::OpenedTab)
						->SetForegroundTab(DetailsTabID)
					)
				)
			);
	}

	FWidgetPreviewToolkit::~FWidgetPreviewToolkit()
	{
		if (GEditor)
		{
			GEditor->OnBlueprintPreCompile().Remove(OnBlueprintPrecompileHandle);
		}

		if (Preview)
		{
			Preview->ClearWidgetInstance();
			Preview->OnWidgetChanged().Remove(OnWidgetChangedHandle);
		}

		// Ensure remaining references to the update state stop ticking
		SetState(&PausedState);

		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().OnFocusChanging().Remove(OnFocusChangingHandle);
		}
	}

	void FWidgetPreviewToolkit::CreateWidgets()
	{
		FBaseAssetToolkit::CreateWidgets();

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogListing = MessageLogModule.GetLogListing(MessageLogName);
		MessageLogWidget = MessageLogModule.CreateLogListingWidget(MessageLogListing.ToSharedRef());

		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");

		const TSharedRef<FWidgetPreviewToolkit>& Self = SharedThis(this);
		PreviewScene = MakeShared<FWidgetPreviewScene>(Self);

		TArray<FAdvancedPreviewSceneModule::FDetailDelegates> Delegates;
		Delegates.Add({ OnPreviewSceneChangedDelegate });
		PreviewSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(
			PreviewScene->GetPreviewScene(),
			nullptr,
			TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>(),  TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>(),
			Delegates);

		if (Preview)
		{
			Preview->GetOrCreateWidgetInstance(GetPreviewWorld(), true);
		}
	}

	void FWidgetPreviewToolkit::RegisterToolbar()
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		FName ParentName;
		const FName MenuName = GetToolMenuToolbarName(ParentName);
		UToolMenus* ToolMenus = UToolMenus::Get();
		UToolMenu* ToolbarMenu = ToolMenus->ExtendMenu(MenuName);
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			ToolbarMenu = ToolMenus->RegisterMenu(MenuName, ParentName, EMultiBoxType::ToolBar);
		}

		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
		const FWidgetPreviewCommands& Commands = FWidgetPreviewCommands::Get();

		// Preview Section
		{
			FToolMenuSection& PreviewSection = ToolbarMenu->FindOrAddSection(
				"Preview",
				{},
				InsertAfterAssetSection);

			PreviewSection.AddEntry(
				FToolMenuEntry::InitToolBarButton(
					Commands.ResetPreview,
					FText::GetEmpty(),
					{},
					FSlateIcon(FWidgetPreviewStyle::Get().GetStyleSetName(), "WidgetPreview.Reset")));
		}
	}

	void FWidgetPreviewToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FBaseAssetToolkit::RegisterTabSpawners(InTabManager);

		ensure(AssetEditorTabsCategory.IsValid());

		const TSharedRef<FWorkspaceItem> AssetEditorTabsCategoryRef = AssetEditorTabsCategory.ToSharedRef();

		InTabManager->RegisterTabSpawner(
			PreviewSceneSettingsTabID,
			FOnSpawnTab::CreateSP(this, &FWidgetPreviewToolkit::SpawnTab_PreviewSceneSettings))
			.SetDisplayName(LOCTEXT("PreviewSceneTab", "Preview Scene Settings"))
			.SetGroup(AssetEditorTabsCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
			.SetReadOnlyBehavior(ETabReadOnlyBehavior::Disabled);

		InTabManager->RegisterTabSpawner(
			MessageLogTabID,
			FOnSpawnTab::CreateSP(this, &FWidgetPreviewToolkit::SpawnTab_MessageLog))
			.SetDisplayName(LOCTEXT("MessageLogTab", "Message Log"))
			.SetGroup(AssetEditorTabsCategoryRef)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "MessageLog.TabIcon"));

		const TSharedRef<IWidgetPreviewToolkit> ToolkitRef = SharedThis(this);

		IUMGWidgetPreviewModule& UMGWidgetPreviewModule = FModuleManager::LoadModuleChecked<IUMGWidgetPreviewModule>("UMGWidgetPreview");
		UMGWidgetPreviewModule.OnRegisterTabsForEditor().Broadcast(ToolkitRef, InTabManager);
	}

	void FWidgetPreviewToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FBaseAssetToolkit::UnregisterTabSpawners(InTabManager);

		InTabManager->UnregisterTabSpawner(PreviewSceneSettingsTabID);
		InTabManager->UnregisterTabSpawner(MessageLogTabID);
	}

	void FWidgetPreviewToolkit::PostInitAssetEditor()
	{
		if (GEditor)
		{
			OnBlueprintPrecompileHandle = GEditor->OnBlueprintPreCompile().AddRaw(this, &FWidgetPreviewToolkit::OnBlueprintPrecompile);
		}

		OnWidgetChangedHandle = Preview->OnWidgetChanged().AddSP(this, &FWidgetPreviewToolkit::OnWidgetChanged);

		if (FSlateApplication::IsInitialized())
		{
			OnFocusChangingHandle = FSlateApplication::Get().OnFocusChanging().AddSP(this, &FWidgetPreviewToolkit::OnFocusChanging);
		}

		// Bind Commands
		{
			const FWidgetPreviewCommands& Commands = FWidgetPreviewCommands::Get();

			ToolkitCommands->MapAction(
				Commands.ResetPreview,
				FExecuteAction::CreateSP(this, &FWidgetPreviewToolkit::ResetPreview));
		}

		ResolveState();
	}

	bool FWidgetPreviewToolkit::CanSaveAsset() const
	{
		// We use the same logic here - if the outer package is transient, the only option is "SaveAs"
		return IsSaveAssetVisible();
	}

	void FWidgetPreviewToolkit::SaveAsset_Execute()
	{
		TArray<UObject*> ObjectsToSave;
		GetSaveableObjects(ObjectsToSave);

		if (ObjectsToSave.Num() == 0)
		{
			return;
		}

		// Check for Transient outer, and if found use SaveAs instead
		for (UObject* Object : ObjectsToSave)
		{
			UPackage* Package = Object->GetPackage();
			if (!Package || Package == GetTransientPackage())
			{
				// Redirect to SaveAs
				SaveAssetAs_Execute();
				return;
			}
		}

		TArray<UObject*> SavedObjects;
		SavedObjects.Reserve(ObjectsToSave.Num());

		TArray<UPackage*> PackagesToSave;

		for (UObject* Object : ObjectsToSave)
		{
			if (Object == nullptr)
			{
				// Log an invalid object but don't try to save it
				UE_LOGF(LogWidgetPreview, Log, "Invalid preview to save: %ls", (Object != nullptr) ? *Object->GetFullName() : TEXT("Null Object"));
			}
			else
			{
				PackagesToSave.Add(Object->GetOutermost());
				SavedObjects.Add(Object);
			}
		}

		constexpr bool bPromptToSave = false;
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirtyOnAssetSave, bPromptToSave);

		OnAssetsSaved(SavedObjects);
	}

	bool FWidgetPreviewToolkit::IsSaveAssetAsVisible() const
	{
		// @note: usually this wouldn't appear when the asset belongs to the transient package
		// We allow this so that the user has the option of saving it to an asset (non-transient package)
		return true;
	}

	void FWidgetPreviewToolkit::SaveAssetAs_Execute()
	{
		TSharedPtr<IToolkitHost> MyToolkitHost = ToolkitHost.Pin();
		if (!MyToolkitHost.IsValid())
		{
			return;
		}

		TArray<UObject*> ObjectsToSave;
		GetSaveableObjects(ObjectsToSave);

		if (ObjectsToSave.Num() == 0)
		{
			return;
		}

		TArray<UObject*> ObjectsToSaveWithoutPackage;
		ObjectsToSaveWithoutPackage.Reserve(ObjectsToSave.Num());

		// Temporarily set to Transient objects, so SaveAssetsAs will auto-populate the default path
		for (UObject* Object : ObjectsToSave)
		{
			UPackage* Package = Object->GetPackage();
			if (!Package || Package == GetTransientPackage())
			{
				Object->SetFlags(Object->GetFlags() | RF_Transient);
				ObjectsToSaveWithoutPackage.Emplace(Object);
			}
		}

		TArray<UObject*> SavedObjects;
		FEditorFileUtils::SaveAssetsAs(ObjectsToSave, SavedObjects);

		if (SavedObjects.Num() == 0)
		{
			// Error saving, or user closed the dialog. Restore objects to non-transient
			for (UObject* Object : ObjectsToSaveWithoutPackage)
            {
				Object->ClearFlags(EObjectFlags::RF_Transient);
            }

			return;
		}

		// close existing asset editors for resaved assets
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

		TArray<UObject*> ObjectsBeingEdited = GetEditingObjects();

		// hack @see: FAssetEditorToolkit::SaveAssetAs_Execute()
		TArray<UObject*> ObjectsToReopen;
		for (UObject* Object : ObjectsBeingEdited)
		{
			if (Object->IsAsset() && !ObjectsToSave.Contains(Object))
			{
				ObjectsToReopen.Add(Object);
			}
		}

		for (UObject* Object : SavedObjects)
		{
			if (ShouldReopenEditorForSavedAsset(Object))
			{
				ObjectsToReopen.AddUnique(Object);
			}
		}

		for (UObject* Object : ObjectsBeingEdited)
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(Object);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyAssetClosed(Object, this);
		}

		AssetEditorSubsystem->OpenEditorForAssets_Advanced(ObjectsToReopen, ToolkitMode, MyToolkitHost.ToSharedRef());
		// end hack

		OnAssetsSavedAs(SavedObjects);
	}

	void FWidgetPreviewToolkit::GetSaveableObjects(TArray<UObject*>& OutObjects) const
	{
		FBaseAssetToolkit::GetSaveableObjects(OutObjects);

		TArray<UObject*> ObjectsBeingEdited = GetEditingObjects();
		for (UObject* Object : ObjectsBeingEdited)
		{
			// We override this to allow Transient objects to be saved
			OutObjects.Add(Object);
		}
	}

	FText FWidgetPreviewToolkit::GetToolkitName() const
	{
		const TArray<UObject*>* Objects = GetObjectsCurrentlyBeingEdited();

		// Singular
		if (Objects->Num() == 1)
		{
			return FText::Format(LOCTEXT("WidgetPreviewTabNameWithObject", "Widget Preview: {0}"), GetLabelForObject((*Objects)[0]));
		}

		// Plural
		return LOCTEXT("WidgetPreviewTabNameWithObjects", "Widget Preview: (Multiple)");
	}

	FName FWidgetPreviewToolkit::GetToolkitFName() const
	{
		return FName(FString::Printf(TEXT("WidgetPreview%p"), this));
	}

	FText FWidgetPreviewToolkit::GetBaseToolkitName() const
	{
		return LOCTEXT("WidgetPreviewToolkitName", "Widget Preview");
	}

	FString FWidgetPreviewToolkit::GetWorldCentricTabPrefix() const
	{
		return LOCTEXT("WorldCentricTabPrefix", "WidgetPreview").ToString();
	}

	FLinearColor FWidgetPreviewToolkit::GetWorldCentricTabColorScale() const
	{
		return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
	}

	void FWidgetPreviewToolkit::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(Preview);
	}

	FString FWidgetPreviewToolkit::GetReferencerName() const
	{
		return TEXT("FWidgetPreviewToolkit");
	}

	TSharedPtr<FLayoutExtender> FWidgetPreviewToolkit::GetLayoutExtender() const
	{
		return LayoutExtender;
	}

	IWidgetPreviewToolkit::FOnSelectedObjectsChanged& FWidgetPreviewToolkit::OnSelectedObjectsChanged()
	{
		return SelectedObjectsChangedDelegate;
	}

	TConstArrayView<TWeakObjectPtr<UObject>> FWidgetPreviewToolkit::GetSelectedObjects() const
	{
		return SelectedObjects;
	}

	void FWidgetPreviewToolkit::SetSelectedObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects)
	{
		if (InObjects.IsEmpty())
		{
			SelectedObjects = { Preview };
		}
		else
		{
			SelectedObjects = InObjects;
		}

		if (SelectedObjectsChangedDelegate.IsBound())
		{
			SelectedObjectsChangedDelegate.Broadcast(SelectedObjects);
		}
	}

	UWidgetPreview* FWidgetPreviewToolkit::GetPreview() const
	{
		return Preview;
	}

	FWidgetPreviewToolkitStateBase* FWidgetPreviewToolkit::GetState() const
	{
		return CurrentState;
	}

	FWidgetPreviewToolkit::FOnStateChanged& FWidgetPreviewToolkit::OnStateChanged()
	{
		return OnStateChangedDelegate;
	}

	FSimpleMulticastDelegate& FWidgetPreviewToolkit::OnMessageChanged()
	{
		return OnMessageChangedDelegate;
	}

	UWorld* FWidgetPreviewToolkit::GetPreviewWorld()
	{
		if (TSharedPtr<FWidgetPreviewScene> PreviewScenePtr = GetPreviewScene())
		{
			return PreviewScenePtr->GetWorld();
		}

		return nullptr;
	}

	TSharedPtr<FWidgetPreviewScene> FWidgetPreviewToolkit::GetPreviewScene()
	{
		if (!PreviewScene.IsValid())
		{
			const TSharedRef<FWidgetPreviewToolkit>& Self = SharedThis(this);
			PreviewScene = MakeShared<FWidgetPreviewScene>(Self);
		}

		return PreviewScene;
	}

	bool FWidgetPreviewToolkit::ShouldUpdate() const
	{
		if (CurrentState)
		{
			return CurrentState->CanTick();
		}

		return bIsFocused;
	}

	void FWidgetPreviewToolkit::OnBlueprintPrecompile(UBlueprint* InBlueprint)
	{
		if (Preview)
		{
			if (const UUserWidget* WidgetCDO = Preview->GetWidgetCDO())
			{
				if (InBlueprint && InBlueprint->GeneratedClass
					&& WidgetCDO->IsA(InBlueprint->GeneratedClass))
				{
					Preview->ClearWidgetInstance();
				}
			}
		}
	}

	void FWidgetPreviewToolkit::OnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType)
	{
		ResolveState();
	}

	void FWidgetPreviewToolkit::OnFocusChanging(
		const FFocusEvent& InFocusEvent,
		const FWeakWidgetPath& InOldWidgetPath, const TSharedPtr<SWidget>& InOldWidget,
		const FWidgetPath& InNewWidgetPath, const TSharedPtr<SWidget>& InNewWidget)
	{
		if (IsHosted())
		{
			const SWidget* ToolkitParentWidget = GetToolkitHost()->GetParentWidget().ToSharedPtr().Get();
			const bool bToolkitInNewWidgetPath = InNewWidgetPath.ContainsWidget(ToolkitParentWidget);
			if (bIsFocused && !bToolkitInNewWidgetPath)
			{
				// Focus lost
				bIsFocused = false;
				ResolveState();
			}
			else if (!bIsFocused && bToolkitInNewWidgetPath)
			{
				// Focus received
				bIsFocused = true;
				ResolveState();
			}
		}
	}

	bool FWidgetPreviewToolkit::SetState(FWidgetPreviewToolkitStateBase* InNewState)
	{
		FWidgetPreviewToolkitStateBase* OldState = CurrentState;
		FWidgetPreviewToolkitStateBase* NewState = InNewState;

		if (OldState == NewState)
		{
			return false;
		}

		if (OldState)
		{
			OldState->OnExit(NewState);
		}

		if (NewState)
		{
			NewState->OnEnter(OldState);
		}

		CurrentState = NewState;
		OnStateChanged().Broadcast(OldState, NewState);

		return true;
	}

	void FWidgetPreviewToolkit::ResolveState()
	{
		FWidgetPreviewToolkitStateBase* NewState = nullptr;
		bool bForceMessageUpdate = false;

		if (!bIsFocused)
		{
			NewState = &BackgroundState;
		}
		else
		{
			TArray<const UUserWidget*> FailedWidgets;
			if (!GetPreview()->CanCallInitializedWithoutPlayerContext(true, FailedWidgets))
			{
				UnsupportedWidgetState.SetUnsupportedWidgets(FailedWidgets);
				NewState = &UnsupportedWidgetState;
				bForceMessageUpdate = true;
			}

			// If we're here, the current state should be valid/running
			if (NewState == nullptr)
			{
				NewState = &RunningState;
			}
		}

		const bool bStateChanged = SetState(NewState);

		if (!bStateChanged && bForceMessageUpdate)
		{
			OnMessageChanged().Broadcast();
		}
	}

	void FWidgetPreviewToolkit::ResetPreview()
	{
		if (Preview)
		{
			// Don't need the returned instance, just need to have it rebuild
			Preview->GetOrCreateWidgetInstance(GetPreviewWorld(), true);
		}
	}

	TSharedRef<SDockTab> FWidgetPreviewToolkit::SpawnTab_Viewport(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == ViewportTabID);

		FAssetEditorViewportConstructionArgs ViewportArgs;
		ViewportArgs.ViewportType = LVT_Perspective;
		ViewportArgs.bRealtime = true;

		const TSharedRef<FWidgetPreviewToolkit>& Self = SharedThis(this);
		TSharedRef<SEditorViewport> ViewportWidget = SNew(SWidgetPreviewViewport, Self);

		TSharedRef<SDockTab> DockTab = SNew(SDockTab);

		DockTab
		->SetContent(
			SNew(SOverlay)
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				ViewportWidget
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SWidgetPreview, Self)
				.IsEnabled(this, &FWidgetPreviewToolkit::ShouldUpdate)
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SWidgetPreviewStatus, Self)
			]
		);

		return DockTab;
	}

	TSharedRef<SDockTab> FWidgetPreviewToolkit::SpawnTab_Details(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == DetailsTabID);

		const TSharedRef<FWidgetPreviewToolkit>& Self = SharedThis(this);

		return SNew(SDockTab)
		[
			SNew(SWidgetPreviewDetails, Self)
		];
	}

	TSharedRef<SDockTab> FWidgetPreviewToolkit::SpawnTab_PreviewSceneSettings(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId().TabType == PreviewSceneSettingsTabID);

		return SNew(SDockTab)
		.Label( LOCTEXT("StaticMeshPreviewScene_TabTitle", "Preview Scene Settings") )
		[
			PreviewSettingsWidget.IsValid() ? PreviewSettingsWidget.ToSharedRef() : SNullWidget::NullWidget
		];
	}

	TSharedRef<SDockTab> FWidgetPreviewToolkit::SpawnTab_MessageLog(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId().TabType == MessageLogTabID);
		return SNew(SDockTab)
		[
			MessageLogWidget.ToSharedRef()
		];
	}
}

#undef LOCTEXT_NAMESPACE
