// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneModule.h"
#include "Framework/Docking/TabManager.h"
#include "IWidgetPreviewToolkit.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/DataValidation/Fixer.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/BaseToolkit.h"
#include "Tools/BaseAssetToolkit.h"
#include "WidgetPreview.h"

class UWidgetPreviewEditor;
class FWidgetBlueprintEditor;
class IMessageLogListing;
class UWidgetPreview;

namespace UE::UMGWidgetPreview::Private
{
	struct FWidgetPreviewabilityFixer : DataValidation::IFixer
	{
		virtual EFixApplicability GetApplicability(int32 FixIndex) const override;
		virtual FFixResult ApplyFix(int32 FixIndex) override;

		TWeakObjectPtr<const UUserWidget> WeakUserWidget;

		static TSharedRef<FWidgetPreviewabilityFixer> Create(const UUserWidget* InUserWidget);
	};

	class FWidgetPreviewToolkit;

	/** Encapsulates the state needed to run the preview world. */
	class FWidgetPreviewScene
		: public FTickableEditorObject
	{
	public:
		explicit FWidgetPreviewScene(const TSharedRef<FWidgetPreviewToolkit>& InPreviewToolkit);
		virtual ~FWidgetPreviewScene() override = default;

		//~ Begin FTickableEditorObject
		virtual void Tick(float DeltaTime) override;
		virtual ETickableTickType GetTickableTickType() const override;
		virtual TStatId GetStatId() const override;
		//~ End FTickableEditorObject

	public:
		UWorld* GetWorld() const;

		TSharedRef<FAdvancedPreviewScene> GetPreviewScene() const;

	private:
		TWeakPtr<FWidgetPreviewToolkit> WeakToolkit;
		TSharedPtr<FAdvancedPreviewScene> PreviewScene;
	};

	struct FWidgetPreviewToolkitStateBase
	{
		explicit FWidgetPreviewToolkitStateBase(const FName& Id);

		virtual ~FWidgetPreviewToolkitStateBase() = default;

		FName GetId() const;
		const TSharedPtr<FTokenizedMessage>& GetStatusMessage() const;
		bool CanTick() const;
		bool ShouldOverlayStatusMessage() const;

		virtual void OnEnter(const FWidgetPreviewToolkitStateBase* InFromState);
		virtual void OnExit(const FWidgetPreviewToolkitStateBase* InToState);

	protected:
		FName Id;
		TSharedPtr<FTokenizedMessage> StatusMessage;
		bool bCanTick = false;
		bool bShouldOverlayMessage = false;
	};

	struct FWidgetPreviewToolkitPausedState : FWidgetPreviewToolkitStateBase
	{
		FWidgetPreviewToolkitPausedState();
	};

	struct FWidgetPreviewToolkitBackgroundState : FWidgetPreviewToolkitPausedState
	{
		FWidgetPreviewToolkitBackgroundState();
	};

	struct FWidgetPreviewToolkitUnsupportedWidgetState : FWidgetPreviewToolkitPausedState
	{
		FWidgetPreviewToolkitUnsupportedWidgetState();

		void SetUnsupportedWidgets(const TArray<const UUserWidget*>& InWidgets);

	private:
		void ResetStatusMessage();

	private:
		TArray<TWeakObjectPtr<const UUserWidget>> UnsupportedWidgets;
	};

	struct FWidgetPreviewToolkitRunningState : FWidgetPreviewToolkitStateBase
	{
		FWidgetPreviewToolkitRunningState();
	};

	class FWidgetPreviewToolkit
		: public FBaseAssetToolkit
		, public FGCObject
		, public IWidgetPreviewToolkit
	{
	public:
		explicit FWidgetPreviewToolkit(UWidgetPreviewEditor* InOwningEditor);
		virtual ~FWidgetPreviewToolkit() override;

		//~ Begin FBaseAssetToolkit
		virtual void CreateWidgets() override;
		virtual void RegisterToolbar() override;
		//~ End FBaseAssetToolkit

		//~ Begin FAssetEditorToolkit
		virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
		virtual void PostInitAssetEditor() override;
		virtual bool CanSaveAsset() const override;
		virtual void SaveAsset_Execute() override;
		virtual bool IsSaveAssetAsVisible() const override;
		virtual void SaveAssetAs_Execute() override;
		virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
		//~ End FAssetEditorToolkit

		//~ Begin IToolkit
		virtual FText GetToolkitName() const override;
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FString GetWorldCentricTabPrefix() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		//~ End IToolkit

		//~ Begin FGCObject
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;
		//~ End FGCObject

		//~ Begin IWidgetPreviewToolkit
		virtual TSharedPtr<FLayoutExtender> GetLayoutExtender() const override;
		virtual FOnSelectedObjectsChanged& OnSelectedObjectsChanged() override;
		virtual TConstArrayView<TWeakObjectPtr<UObject>> GetSelectedObjects() const override;
		virtual void SetSelectedObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects) override;
		virtual UWidgetPreview* GetPreview() const override;
		virtual UWorld* GetPreviewWorld() override;
		//~ End IWidgetPreviewToolkit

		FWidgetPreviewToolkitStateBase* GetState() const;

		using FOnStateChanged = TMulticastDelegate<void(FWidgetPreviewToolkitStateBase* InOldState, FWidgetPreviewToolkitStateBase* InNewState)>;
		FOnStateChanged& OnStateChanged();
		FSimpleMulticastDelegate& OnMessageChanged();

		TSharedPtr<FWidgetPreviewScene> GetPreviewScene();

	public:
		static const FLazyName PreviewSceneSettingsTabID;
		static const FLazyName MessageLogTabID;

	protected:
		bool ShouldUpdate() const;

		void OnBlueprintPrecompile(UBlueprint* InBlueprint);

		void OnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType);

		void OnFocusChanging(
			const FFocusEvent& InFocusEvent,
			const FWeakWidgetPath& InOldWidgetPath, const TSharedPtr<SWidget>& InOldWidget,
			const FWidgetPath& InNewWidgetPath, const TSharedPtr<SWidget>& InNewWidget);

		/** If the given state is different to the current state, this will handle transitions and events. Returns true if the state changed. */
		bool SetState(FWidgetPreviewToolkitStateBase* InNewState);

		/** Resolve and set the current state based on various conditions. */
		void ResolveState();

		/** Resets to the default state. */
		void ResetPreview();

		virtual TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args) override;
		virtual TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args) override;
		TSharedRef<SDockTab> SpawnTab_PreviewSceneSettings(const FSpawnTabArgs& Args);
		TSharedRef<SDockTab> SpawnTab_MessageLog(const FSpawnTabArgs& Args);

	private:
		TObjectPtr<UWidgetPreview> Preview;

		FOnSelectedObjectsChanged SelectedObjectsChangedDelegate;
		TArray<TWeakObjectPtr<UObject>> SelectedObjects;

		TSharedPtr<FWidgetPreviewScene> PreviewScene;
		FAdvancedPreviewSceneModule::FOnPreviewSceneChanged OnPreviewSceneChangedDelegate;
		TSharedPtr<SWidget> PreviewSettingsWidget;

		TSharedPtr<IMessageLogListing> MessageLogListing;
		TSharedPtr<SWidget> MessageLogWidget;

		bool bIsFocused = false;

		FDelegateHandle OnBlueprintPrecompileHandle;
		FDelegateHandle OnWidgetChangedHandle;
		FDelegateHandle OnFocusChangingHandle;
		FOnStateChanged OnStateChangedDelegate;
		FSimpleMulticastDelegate OnMessageChangedDelegate; // Called when state doesn't change, but the message still needs to update.

		FWidgetPreviewToolkitStateBase* CurrentState = nullptr;
		FWidgetPreviewToolkitPausedState PausedState;
		FWidgetPreviewToolkitBackgroundState BackgroundState;
		FWidgetPreviewToolkitUnsupportedWidgetState UnsupportedWidgetState;
		FWidgetPreviewToolkitRunningState RunningState;
	};
}
