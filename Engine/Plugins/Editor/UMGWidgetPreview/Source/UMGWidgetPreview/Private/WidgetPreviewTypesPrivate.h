// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "WidgetBlueprint.h"

namespace UE::UMGWidgetPreview::Private
{
	struct FWidgetTypeTuple
	{
		FWidgetTypeTuple() = default;

		explicit FWidgetTypeTuple(const UUserWidget* InUserWidgetCDO)
			: bIsValid(true)
		{
			Set(InUserWidgetCDO);
		}

		/** Attempt to resolve the tuple from the given UserWidget CDO. */
		void Set(const UUserWidget* InUserWidgetCDO)
		{
			check(InUserWidgetCDO);

			ClassDefaultObject = InUserWidgetCDO;
			BlueprintGeneratedClass = Cast<UWidgetBlueprintGeneratedClass>(ClassDefaultObject->GetClass());
			Blueprint = Cast<UWidgetBlueprint>(BlueprintGeneratedClass->ClassGeneratedBy);
		}

		const UUserWidget* ClassDefaultObject = nullptr;
		UWidgetBlueprint* Blueprint = nullptr;
		UWidgetBlueprintGeneratedClass* BlueprintGeneratedClass = nullptr;

		/** Returns true if at least one of the tuple values is valid. */
		inline bool IsValid() const
		{
			return bIsValid && (ClassDefaultObject || Blueprint || BlueprintGeneratedClass);
		}

	private:
		/** True only when value resolution has been attempted. */
		bool bIsValid = false;
	};
}
