// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWidgetPreview.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "WidgetPreview.h"
#include "Slate/SRetainerWidget.h"
#include "WidgetPreviewToolkit.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "SWidgetPreview"

namespace UE::UMGWidgetPreview::Private
{
	void SWidgetPreview::Construct(const FArguments& Args, const TSharedRef<FWidgetPreviewToolkit>& InToolkit)
	{
		WeakToolkit = InToolkit;

		OnStateChangedHandle = InToolkit->OnStateChanged().AddSP(this, &SWidgetPreview::OnStateChanged);
		OnWidgetChangedHandle = InToolkit->GetPreview()->OnWidgetChanged().AddSP(this, &SWidgetPreview::OnWidgetChanged);

		CreatedSlateWidget = SNullWidget::NullWidget;

		ContainerWidget = SNew(SBorder)
		[
			GetCreatedSlateWidget()
		];

		ChildSlot
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.Expose(OverlaySlot)
			[
				SAssignNew(SizeBoxWidget, SBox)
				[
					SAssignNew(RetainerWidget, SRetainerWidget)
					.RenderOnPhase(false)
					.RenderOnInvalidation(false)
					[
						ContainerWidget.ToSharedRef()
					]
				]
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(0.0f, 0.0f, 6.0f, 2.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.Text(this, &SWidgetPreview::GetPreviewSizeText)
				.ColorAndOpacity(FSlateColor(FLinearColor(1, 1, 1, 0.25f)))
			]
		];

		OnWidgetChanged(EWidgetPreviewWidgetChangeType::Assignment);

	}

	SWidgetPreview::~SWidgetPreview()
	{
		ContainerWidget->ClearContent();

		if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			Toolkit->OnStateChanged().Remove(OnStateChangedHandle);

			if (UWidgetPreview* Preview = Toolkit->GetPreview())
			{
				Preview->OnWidgetChanged().Remove(OnWidgetChangedHandle);
			}
		}
	}

	int32 SWidgetPreview::OnPaint(
		const FPaintArgs& Args,
		const FGeometry& AllottedGeometry,
		const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FWidgetStyle& InWidgetStyle,
		bool bParentEnabled) const
	{
		const int32 Result = SCompoundWidget::OnPaint(
			Args,
			AllottedGeometry,
			MyCullingRect,
			OutDrawElements,
			LayerId,
			InWidgetStyle,
			bParentEnabled);

		if (bClearWidgetOnNextPaint)
		{
			SWidgetPreview* MutableThis = const_cast<SWidgetPreview*>(this);

			MutableThis->CreatedSlateWidget = SNullWidget::NullWidget;
			ContainerWidget->SetContent(GetCreatedSlateWidget());
			MutableThis->bClearWidgetOnNextPaint = false;
		}

		return Result;
	}

	void SWidgetPreview::OnStateChanged(FWidgetPreviewToolkitStateBase* InOldState, FWidgetPreviewToolkitStateBase* InNewState)
	{
		const bool bShouldUseLiveWidget = InNewState->CanTick();
		bIsRetainedRender = !bShouldUseLiveWidget;
		bClearWidgetOnNextPaint = bIsRetainedRender;
		RetainerWidget->RequestRender();
		RetainerWidget->SetRetainedRendering(bIsRetainedRender);

		if (bShouldUseLiveWidget)
		{
			OnWidgetChanged(EWidgetPreviewWidgetChangeType::Assignment);
		}
	}

	void SWidgetPreview::OnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType)
	{
		// Disallow widget assignment if retaining (cached thumbnail)
		if (bIsRetainedRender)
		{
			return;
		}

		if (InChangeType == EWidgetPreviewWidgetChangeType::Resized)
		{
			if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
			{
				if (UWidgetPreview* Preview = Toolkit->GetPreview())
				{
					// Override with custom size if set
					bSizeOverridden = Preview->GetbShouldOverrideWidgetSize();
					if (bSizeOverridden)
					{
						CreatedWidgetPreviewSizeMode = EDesignPreviewSizeMode::Custom;
						CreatedWidgetPreviewCustomSize = Preview->GetOverriddenWidgetSize();
					}
					else if (!Preview->GetWidgetType().ObjectPath.IsNull())
					{
						if (const UUserWidget* WidgetCDO = Preview->GetWidgetCDO())
						{
							CreatedWidgetPreviewSizeMode = WidgetCDO->DesignSizeMode;
							CreatedWidgetPreviewCustomSize = WidgetCDO->DesignTimeSize;
						}
					}
					RebuildContainer();
				}
			}
		}
		else if (InChangeType != EWidgetPreviewWidgetChangeType::Destroyed)
		{
			if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
			{
				if (UWidgetPreview* Preview = Toolkit->GetPreview())
				{
					UWorld* World = GetWorld();
					bool bHasValidWidget = true;
					if (const TSharedPtr<SWidget> PreviewSlateWidget = Preview->GetSlateWidgetInstance())
					{
						CreatedSlateWidget = PreviewSlateWidget;
					}
					else if (UUserWidget* PreviewWidget = Preview->GetOrCreateWidgetInstance(World))
					{
						CreatedSlateWidget = PreviewWidget->TakeWidget();
					}
					else
					{
						CreatedSlateWidget = SNullWidget::NullWidget;
						bHasValidWidget = false;
					}

					// Override with custom size if set
					bSizeOverridden = Preview->GetbShouldOverrideWidgetSize();
					if (bHasValidWidget && !bSizeOverridden)
					{
						if (const UUserWidget* WidgetCDO = Preview->GetWidgetCDO())
						{
							CreatedWidgetPreviewSizeMode = WidgetCDO->DesignSizeMode;
							CreatedWidgetPreviewCustomSize = WidgetCDO->DesignTimeSize;
						}
					}

					RebuildContainer();
					ContainerWidget->SetContent(GetCreatedSlateWidget());
				}
			}
		}
	}

	UWorld* SWidgetPreview::GetWorld() const
	{
		if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			return Toolkit->GetPreviewWorld();
		}

		return nullptr;
	}

	TSharedRef<SWidget> SWidgetPreview::GetCreatedSlateWidget() const
	{
		if (TSharedPtr<SWidget> SlateWidget = CreatedSlateWidget.Pin())
		{
			return SlateWidget.ToSharedRef();
		}

		return SNullWidget::NullWidget;
	}

	void SWidgetPreview::RebuildContainer()
	{
		switch (CreatedWidgetPreviewSizeMode)
		{
			case EDesignPreviewSizeMode::Custom:
			case EDesignPreviewSizeMode::CustomOnScreen:
			{
				if (SizeBoxWidget.IsValid())
				{
					SizeBoxWidget->SetWidthOverride(CreatedWidgetPreviewCustomSize.X);
					SizeBoxWidget->SetHeightOverride(CreatedWidgetPreviewCustomSize.Y);
				}
				OverlaySlot->SetHorizontalAlignment(HAlign_Center);
				OverlaySlot->SetVerticalAlignment(VAlign_Center);
				break;
			}
			case EDesignPreviewSizeMode::Desired:
			case EDesignPreviewSizeMode::DesiredOnScreen:
			{
				if (SizeBoxWidget.IsValid())
				{
					SizeBoxWidget->SetWidthOverride(FOptionalSize());
					SizeBoxWidget->SetHeightOverride(FOptionalSize());
				}
				OverlaySlot->SetHorizontalAlignment(HAlign_Center);
				OverlaySlot->SetVerticalAlignment(VAlign_Center);
				break;
			}
			case EDesignPreviewSizeMode::FillScreen:
			{
				if (SizeBoxWidget.IsValid())
				{
					SizeBoxWidget->SetWidthOverride(FOptionalSize());
					SizeBoxWidget->SetHeightOverride(FOptionalSize());
				}
				OverlaySlot->SetHorizontalAlignment(HAlign_Fill);
				OverlaySlot->SetVerticalAlignment(VAlign_Fill);
				break;
			}
			default:
			{
				checkf(false, TEXT("Invalid Widget Preview Size Mode"));
				break;
			}
		}
	}

	FText SWidgetPreview::GetPreviewSizeText() const
	{
		

		switch (CreatedWidgetPreviewSizeMode)
		{
			case EDesignPreviewSizeMode::Custom:
			case EDesignPreviewSizeMode::CustomOnScreen:
			{
				FNumberFormattingOptions Options = FNumberFormattingOptions::DefaultNoGrouping();

				if (bSizeOverridden)
				{
					return FText::Format(LOCTEXT("PreviewSizeTextOverriden", "Preview Size: Overriden ({0}, {1})"),
						FText::AsNumber(CreatedWidgetPreviewCustomSize.X, &FNumberFormattingOptions::DefaultNoGrouping()),
						FText::AsNumber(CreatedWidgetPreviewCustomSize.Y, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				else
				{
					return FText::Format(LOCTEXT("PreviewSizeTextCustom", "Preview Size: Custom ({0}, {1})"),
						FText::AsNumber(CreatedWidgetPreviewCustomSize.X, &FNumberFormattingOptions::DefaultNoGrouping()),
						FText::AsNumber(CreatedWidgetPreviewCustomSize.Y, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
			}
			case EDesignPreviewSizeMode::Desired:
			case EDesignPreviewSizeMode::DesiredOnScreen:
			{
				return FText(LOCTEXT("PreviewSizeTextDesired", "Preview Size: Desired"));
			}
			case EDesignPreviewSizeMode::FillScreen:
			{
				return FText(LOCTEXT("PreviewSizeTextFill", "Preview Size: Fill Screen"));
			}
			default:
			{
				checkf(false, TEXT("Invalid Widget Preview Size Mode"));
				break;
			}
		}


		return FText();
	}

}

#undef LOCTEXT_NAMESPACE
