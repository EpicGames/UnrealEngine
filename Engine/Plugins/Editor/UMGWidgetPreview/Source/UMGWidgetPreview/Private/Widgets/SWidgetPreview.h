// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "WidgetPreview.h"
#include "Widgets/SCompoundWidget.h"

class UWidget;
class UUserWidget;
class SBorder;
class SRetainerWidget;

namespace UE::UMGWidgetPreview::Private
{
	struct FWidgetPreviewToolkitStateBase;
	class FWidgetPreviewToolkit;

	class SWidgetPreview
		: public SCompoundWidget
	{
	public:
		using FSlotWidgetMap = TMap<FName, TObjectPtr<UWidget>>;

		SLATE_BEGIN_ARGS(SWidgetPreview) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& Args, const TSharedRef<FWidgetPreviewToolkit>& InToolkit);

		virtual ~SWidgetPreview() override;

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	private:
		void OnStateChanged(FWidgetPreviewToolkitStateBase* InOldState, FWidgetPreviewToolkitStateBase* InNewState);
		void OnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType);

		/** Convenience method to get world from the associated viewport. */
		UWorld* GetWorld() const;

		TSharedRef<SWidget> GetCreatedSlateWidget() const;

		void RebuildContainer();
		FText GetPreviewSizeText() const;

	private:
		TWeakPtr<FWidgetPreviewToolkit> WeakToolkit;

		TSharedPtr<SRetainerWidget> RetainerWidget;
		TSharedPtr<SBorder> ContainerWidget;
		TSharedPtr<SBox> SizeBoxWidget;
		TWeakPtr<SWidget> CreatedSlateWidget;
		SOverlay::FOverlaySlot* OverlaySlot;

		bool bClearWidgetOnNextPaint = false;
		bool bIsRetainedRender = false;
		bool bSizeOverridden = false;

		FDelegateHandle OnStateChangedHandle;
		FDelegateHandle OnWidgetChangedHandle;

		EDesignPreviewSizeMode CreatedWidgetPreviewSizeMode;
		FVector2D CreatedWidgetPreviewCustomSize;
	};
}
