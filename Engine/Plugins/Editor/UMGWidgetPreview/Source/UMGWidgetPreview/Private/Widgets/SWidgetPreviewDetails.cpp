// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWidgetPreviewDetails.h"

#include "Blueprint/UserWidget.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "WidgetPreview.h"
#include "WidgetPreviewToolkit.h"

namespace UE::UMGWidgetPreview::Private
{
	void SWidgetPreviewDetails::Construct(const FArguments& Args, const TSharedRef<FWidgetPreviewToolkit>& InToolkit)
	{
		WeakToolkit = InToolkit;

		OnSelectedObjectsChangedHandle = InToolkit->OnSelectedObjectsChanged().AddSP(this, &SWidgetPreviewDetails::OnSelectedObjectChanged);

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.NotifyHook = this;

		DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		OnSelectedObjectChanged({});

		ChildSlot
		[
			DetailsView.ToSharedRef()
		];
	}

	SWidgetPreviewDetails::~SWidgetPreviewDetails()
	{
		if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			Toolkit->OnSelectedObjectsChanged().Remove(OnSelectedObjectsChangedHandle);
		}
	}

	void SWidgetPreviewDetails::OnSelectedObjectChanged(const TConstArrayView<TWeakObjectPtr<UObject>> InSelectedObjects) const
	{
		if (InSelectedObjects.IsEmpty())
		{
			if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
			{
				if (UWidgetPreview* Preview = Toolkit->GetPreview())
				{
					DetailsView->SetObject(Preview);
				}
				else
				{
					DetailsView->SetObject(nullptr);
				}
			}
		}
		else
		{
			TArray<UObject*> SelectedObjects;
			Algo::TransformIf(
				InSelectedObjects,
				SelectedObjects,
				[](const TWeakObjectPtr<UObject>& InWeakObject)
				{
					return InWeakObject.IsValid();
				},
				[](const TWeakObjectPtr<UObject>& InWeakObject)
				{
					return InWeakObject.Get();
				});

			DetailsView->SetObjects(SelectedObjects);
		}
	}

	void SWidgetPreviewDetails::NotifyPostChange(
		const FPropertyChangedEvent& PropertyChangedEvent,
		FEditPropertyChain* PropertyThatChanged)
	{
		for (int32 ObjectIdx = 0; ObjectIdx < PropertyChangedEvent.GetNumObjectsBeingEdited(); ++ObjectIdx)
		{
			if (const UObject* Object = PropertyChangedEvent.GetObjectBeingEdited(ObjectIdx))
			{
				if (PropertyThatChanged->GetActiveMemberNode())
				{
					const FProperty* Property = PropertyThatChanged->GetActiveMemberNode()->GetValue();
					if (Property && Object->GetClass()->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
					{
						TScriptInterface<INotifyFieldValueChanged> Interface = const_cast<UObject*>(Object);
						UE::FieldNotification::FFieldId FieldId = Interface->GetFieldNotificationDescriptor().GetField(Object->GetClass(), Property->GetFName());
						if (FieldId.IsValid())
						{
							Interface->BroadcastFieldValueChanged(FieldId);
						}
					}
				}
			}
		}
	}
}
