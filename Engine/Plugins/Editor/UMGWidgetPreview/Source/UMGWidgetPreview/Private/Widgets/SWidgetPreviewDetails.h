// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "WidgetPreview.h"
#include "Widgets/SCompoundWidget.h"

class FWidgetBlueprintEditor;
class IDetailsView;
class UWidgetPreview;

namespace UE::UMGWidgetPreview::Private
{
	class FWidgetPreviewToolkit;

	class SWidgetPreviewDetails
		: public SCompoundWidget
		, public FNotifyHook
	{
		SLATE_BEGIN_ARGS(SWidgetPreviewDetails) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& Args, const TSharedRef<FWidgetPreviewToolkit>& InToolkit);

		virtual ~SWidgetPreviewDetails() override;

		virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;

	private:
		void OnSelectedObjectChanged(const TConstArrayView<TWeakObjectPtr<UObject>> InSelectedObjects) const;

	private:
		TWeakPtr<FWidgetPreviewToolkit> WeakToolkit;
		TSharedPtr<IDetailsView> DetailsView;

		FDelegateHandle OnSelectedObjectsChangedHandle;
	};
}
