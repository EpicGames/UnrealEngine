// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWidgetPreviewStatus.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "WidgetPreviewToolkit.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "WidgetPreviewStatus"

namespace UE::UMGWidgetPreview::Private
{
	void SWidgetPreviewStatus::Construct(const FArguments& Args, const TSharedRef<FWidgetPreviewToolkit>& InToolkit)
	{
		WeakToolkit = InToolkit;

		OnStateChangedHandle = InToolkit->OnStateChanged().AddSP(this, &SWidgetPreviewStatus::OnStateChanged);
		OnMessageChangedHandle = InToolkit->OnMessageChanged().AddSP(this, &SWidgetPreviewStatus::OnMessageChanged);

		MessageContainerWidget = SNew(SBox);
		MessageContainerWidget->SetContent(MakeMessageWidget());

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(new FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f, FStyleColors::Hover, 1.0f))
			.Visibility(this, &SWidgetPreviewStatus::GetStatusVisibility)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(20.0f, 20.0f, 12.0f, 20.0f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(this, &SWidgetPreviewStatus::GetSeverityIconBrush)
				]

				+ SHorizontalBox::Slot()
				.Padding(0.0f, 20.0f, 8.0f, 20.0f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Center)
				.AutoWidth()
				[
					MessageContainerWidget.ToSharedRef()
				]
			]
		];
	}

	SWidgetPreviewStatus::~SWidgetPreviewStatus()
	{
		if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			Toolkit->OnStateChanged().Remove(OnStateChangedHandle);
			Toolkit->OnMessageChanged().Remove(OnMessageChangedHandle);
		}
	}

	void SWidgetPreviewStatus::OnStateChanged(FWidgetPreviewToolkitStateBase* InOldState, FWidgetPreviewToolkitStateBase* InNewState)
	{
		MessageContainerWidget->SetContent(MakeMessageWidget());
	}

	void SWidgetPreviewStatus::OnMessageChanged()
	{
		MessageContainerWidget->SetContent(MakeMessageWidget());
	}

	TSharedRef<SWidget> SWidgetPreviewStatus::MakeMessageWidget()
	{
		if (const TSharedPtr<FTokenizedMessage>& StatusMessage = GetStatusMessage())
		{
			TSharedRef<SVerticalBox> Container = SNew(SVerticalBox);

			TSharedPtr<SHorizontalBox> RowContainer = nullptr;
			float RowPadding = 0.0f; // Initially 0.0, 8.0 for subsequent rows

			const TArray<TSharedRef<IMessageToken>>& MessageTokens = StatusMessage->GetMessageTokens();
			for (TSharedRef<IMessageToken> MessageToken : MessageTokens)
			{
				const EMessageToken::Type TokenType = MessageToken->GetType();
				switch (TokenType)
				{
				case EMessageToken::Severity:
					// Already handled, skip
					break;

				case EMessageToken::Fix:
					{
						TSharedRef<FFixToken> FixToken = StaticCastSharedRef<FFixToken>(MessageToken);

						RowContainer->AddSlot()
						.HAlign(HAlign_Right)
						[
							SNew(SButton)
							.Text(LOCTEXT("WidgetPreviewStatus_FixToken_Label", "Apply Fix"))
							.ToolTipText(LOCTEXT("WidgetPreviewStatus_FixToken_Tooltip", "Apply the suggested fix."))
							.OnClicked(FOnClicked::CreateSPLambda(
								AsShared(),
								[this, FixToken]
								{
									FFixResult FixResult = FixToken->GetFixer()->ApplyFix(FixToken->GetFixIndex());
									return FReply::Handled();
								}))
						];
					}
					break;

				default:
				case EMessageToken::Text:
					{
						// All other tokens, including text, always denote the start of a new row
						RowContainer = SNew(SHorizontalBox);
						Container->AddSlot()
						.AutoHeight()
						.Padding(0.0f, RowPadding, 0.0f, 0.0f)
						[
							RowContainer.ToSharedRef()
						];
						RowPadding = 8.0f;

						RowContainer->AddSlot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(MessageToken->ToText())
						];
					}
					break;
				}
			}

			return Container;
		}

		return SNullWidget::NullWidget;
	}

	TSharedPtr<FTokenizedMessage> SWidgetPreviewStatus::GetStatusMessage() const
	{
		if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			if (FWidgetPreviewToolkitStateBase* CurrentState = Toolkit->GetState())
			{
				return CurrentState->GetStatusMessage();
			}
		}

		return nullptr;
	}

	EVisibility SWidgetPreviewStatus::GetStatusVisibility() const
	{
		if (const TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			if (FWidgetPreviewToolkitStateBase* CurrentState = Toolkit->GetState())
			{
				return CurrentState->ShouldOverlayStatusMessage() ? EVisibility::Visible : EVisibility::Collapsed;
			}
		}

		return EVisibility::Collapsed;
	}

	const FSlateBrush* SWidgetPreviewStatus::GetSeverityIconBrush() const
	{
		switch (GetSeverity())
		{
		case EMessageSeverity::Error:
			return FAppStyle::Get().GetBrush("Icons.ErrorWithColor");

		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
			return FAppStyle::Get().GetBrush("Icons.WarningWithColor");

		default:
		case EMessageSeverity::Info:
			return FAppStyle::Get().GetBrush("Icons.InfoWithColor");
		}
	}

	EMessageSeverity::Type SWidgetPreviewStatus::GetSeverity() const
	{
		if (const TSharedPtr<FTokenizedMessage>& StatusMessage = GetStatusMessage())
		{
			return StatusMessage->GetSeverity();
		}

		return EMessageSeverity::Info;
	}

	FText SWidgetPreviewStatus::GetMessage() const
	{
		if (const TSharedPtr<FTokenizedMessage>& StatusMessage = GetStatusMessage())
		{
			return StatusMessage->ToText();
		}

		return FText::GetEmpty();
	}
}

#undef LOCTEXT_NAMESPACE
