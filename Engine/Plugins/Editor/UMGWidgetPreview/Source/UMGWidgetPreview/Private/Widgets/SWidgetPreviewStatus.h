// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "WidgetPreviewToolkit.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBox.h"

class UWidgetPreview;
class FWidgetBlueprintEditor;
class FWidgetPreviewToolkit;
class IDetailsView;

namespace UE::UMGWidgetPreview::Private
{
	class SWidgetPreviewStatus
		: public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SWidgetPreviewStatus) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& Args, const TSharedRef<FWidgetPreviewToolkit>& InToolkit);

		virtual ~SWidgetPreviewStatus() override;

	private:
		void OnStateChanged(FWidgetPreviewToolkitStateBase* InOldState, FWidgetPreviewToolkitStateBase* InNewState);
		void OnMessageChanged();

		TSharedRef<SWidget> MakeMessageWidget();

		TSharedPtr<FTokenizedMessage> GetStatusMessage() const;

		EVisibility GetStatusVisibility() const;
		const FSlateBrush* GetSeverityIconBrush() const;
		EMessageSeverity::Type GetSeverity() const;
		FText GetMessage() const;

	private:
		TWeakPtr<FWidgetPreviewToolkit> WeakToolkit;
		TSharedPtr<SBox> MessageContainerWidget;

		FDelegateHandle OnStateChangedHandle;
		FDelegateHandle OnMessageChangedHandle;
	};
}
