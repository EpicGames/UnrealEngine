// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWidgetPreviewViewport.h"

#include "WidgetPreviewToolkit.h"

namespace UE::UMGWidgetPreview::Private
{
	void SWidgetPreviewViewport::Construct(const FArguments& InArgs, const TSharedRef<FWidgetPreviewToolkit>& InToolkit)
	{
		WeakToolkit = InToolkit;

		SEditorViewport::Construct(
			SEditorViewport::FArguments()
			.ViewportSize(InArgs._ViewportSize));
	}

	TSharedRef<FEditorViewportClient> SWidgetPreviewViewport::MakeEditorViewportClient()
	{
		FPreviewScene* PreviewScene = nullptr;
		if (TSharedPtr<FWidgetPreviewToolkit> Toolkit = WeakToolkit.Pin())
		{
			PreviewScene = &Toolkit->GetPreviewScene()->GetPreviewScene().Get();
		}

		TSharedRef<FEditorViewportClient> ViewportClient = MakeShared<FEditorViewportClient>(nullptr, PreviewScene);
		ViewportClient->SetViewLocation(FVector::ZeroVector);
		ViewportClient->SetViewRotation(FRotator(-25.0f, -135.0f, 0.0f));
		ViewportClient->SetViewLocationForOrbiting(FVector::ZeroVector, 500);
		ViewportClient->bSetListenerPosition = false;
		ViewportClient->EngineShowFlags.EnableAdvancedFeatures();
		ViewportClient->EngineShowFlags.SetGrid(true);
		ViewportClient->EngineShowFlags.SetLighting(true);
		ViewportClient->EngineShowFlags.SetIndirectLightingCache(true);
		ViewportClient->EngineShowFlags.SetPostProcessing(true);

		// @todo: temp settings, get from elsewhere?
		ViewportClient->ExposureSettings.bFixed = true;

		ViewportClient->Invalidate();

		return ViewportClient;
	}
}
