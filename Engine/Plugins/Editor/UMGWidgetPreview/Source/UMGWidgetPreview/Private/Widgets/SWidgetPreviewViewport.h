// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SAssetEditorViewport.h"

namespace UE::UMGWidgetPreview::Private
{
	class FWidgetPreviewToolkit;

	class SWidgetPreviewViewport
		: public SEditorViewport
	{
	public:
		SLATE_BEGIN_ARGS(SWidgetPreviewViewport) {}
		SLATE_ATTRIBUTE(FVector2D, ViewportSize);
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<FWidgetPreviewToolkit>& InToolkit);

	private:
		virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;

	private:
		TWeakPtr<FWidgetPreviewToolkit> WeakToolkit;
	};
}
