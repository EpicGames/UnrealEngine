// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

extern const FLazyName UMGWidgetPreviewAppIdentifier;

namespace UE::UMGWidgetPreview
{
	class IWidgetPreviewToolkit;
}

/** The public interface of the UMG widget preview (editor) module. */
class IUMGWidgetPreviewModule
	: public IModuleInterface
{
public:
	DECLARE_EVENT_TwoParams(IUMGWidgetPreviewModule, FOnRegisterTabs, const TSharedPtr<UE::UMGWidgetPreview::IWidgetPreviewToolkit>&, const TSharedRef<FTabManager>&);
	virtual FOnRegisterTabs& OnRegisterTabsForEditor() = 0;
};
