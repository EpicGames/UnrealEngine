// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/BaseToolkit.h"

class UWidgetPreview;

namespace UE::UMGWidgetPreview
{
	class IWidgetPreviewToolkit
	{
	public:
		virtual ~IWidgetPreviewToolkit() = default;

		virtual TSharedPtr<FLayoutExtender> GetLayoutExtender() const = 0;

		using FOnSelectedObjectsChanged = TMulticastDelegate<void(const TConstArrayView<TWeakObjectPtr<UObject>>)>;
		virtual FOnSelectedObjectsChanged& OnSelectedObjectsChanged() = 0;

		/** Gets the "selected" objects, primarily used to display details for a given object/s other than the preview asset itself. */
		virtual TConstArrayView<TWeakObjectPtr<UObject>> GetSelectedObjects() const = 0;

		/** Sets the selected objects. When empty, this will reset to the Preview asset. */
		virtual void SetSelectedObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects) = 0;

		virtual UWidgetPreview* GetPreview() const = 0;

		/** Returns the Preview World. */
		virtual UWorld* GetPreviewWorld() = 0;
	};
}
