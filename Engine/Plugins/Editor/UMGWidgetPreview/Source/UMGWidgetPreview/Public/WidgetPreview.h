// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "WidgetPreview.generated.h"

#define UE_API UMGWIDGETPREVIEW_API

class SWidget;
class UPanelWidget;
class UUserWidget;
class UWidget;
class UWidgetBlueprint;
class UWidgetPreview;
struct FImage;

enum class EWidgetPreviewWidgetChangeType : uint8
{
	Assignment = 0,
	Reinstanced = 1,
	Structure = 2,
	ChildReference = 3,
	Destroyed = 4,				// Just before the Slate widget is destroyed, etc.
	Resized = 5
};

USTRUCT(BlueprintType)
struct FPreviewableWidgetVariant
{
	GENERATED_BODY();

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Widget", DisplayName = "Widget Type", meta = (AllowedClasses = "/Script/UMGEditor.WidgetBlueprint, /Script/UMGWidgetPreview.WidgetPreview"))
	FSoftObjectPath ObjectPath;

	FPreviewableWidgetVariant() = default;
	UE_API explicit FPreviewableWidgetVariant(const TSubclassOf<UUserWidget>& InWidgetType);
	UE_API explicit FPreviewableWidgetVariant(const UWidgetPreview* InWidgetPreview);

public:
	/** Flushes cached widgets and re-resolves from the ObjectPath. */
	UE_API void UpdateCachedWidget();

	/** Returns the referenced Object as a UUserWidget (CDO). Returns nullptr if not found, or we couldn't find a nested UUserWidget (ie. inside a UWidgetPreview). */
	UE_API const UUserWidget* AsUserWidgetCDO() const;

	/** Returns the referenced Object as a UWidgetPreview. Returns nullptr if not found, or not a UWidgetPreview. */
	UE_API const UWidgetPreview* AsWidgetPreview() const;

	friend bool operator==(const FPreviewableWidgetVariant& Left, const FPreviewableWidgetVariant& Right)
	{
		return Left.ObjectPath == Right.ObjectPath;
	}

	friend bool operator!=(const FPreviewableWidgetVariant& Left, const FPreviewableWidgetVariant& Right)
	{
		return !(Left == Right);
	}

private:
	UPROPERTY(Transient)
	TObjectPtr<const UUserWidget> CachedWidgetCDO;

	UPROPERTY(Transient)
	TWeakObjectPtr<UWidgetPreview> CachedWidgetPreview;
};

UCLASS(MinimalAPI, BlueprintType, NotBlueprintable, AutoExpandCategories = "Widgets")
class UWidgetPreview
	: public UObject
{
	GENERATED_BODY()

public:

	UE_API UWidgetPreview(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void BeginDestroy() override;

	/** Convenience function to check that all utilized widgets have bCanCallInitializedWithoutPlayerContext set to true, and reports any that don't. */
	UE_API bool CanCallInitializedWithoutPlayerContext(const bool bInRecursive, TArray<const UUserWidget*>& OutFailedWidgets);

	// @todo: move to utility func somewhere else?
	/** Convenience function to check that the provided widget (and it's children) has bCanCallInitializedWithoutPlayerContext set to true, and reports any that don't. */
	static UE_API bool CanCallInitializedWithoutPlayerContextOnWidget(const UUserWidget* InUserWidget, const bool bInRecursive, TArray<const UUserWidget*>& OutFailedWidgets);

public:
	using FOnWidgetChanged = TMulticastDelegate<void(const EWidgetPreviewWidgetChangeType)>;

	FOnWidgetChanged& OnWidgetChanged() { return OnWidgetChangedDelegate; }

	UFUNCTION(BlueprintCallable, Category = "Layout")
	UE_API const TArray<FName>& GetWidgetSlotNames() const;

	/** Returns or builds and returns an instance of the root widget for previewing. Can be used to trigger a rebuild. */
	[[maybe_unused]] UE_API UUserWidget* GetOrCreateWidgetInstance(UWorld* InWorld, const bool bInForceRecreate = false);

	/** Returns the current widget instance, if any. */
	UE_API UUserWidget* GetWidgetInstance() const;

	/** Returns the current underlying slate widget instance, if any. */
	UE_API TSharedPtr<SWidget> GetSlateWidgetInstance() const;

	/** Stores the current instance in PreviousWidgetInstance, and clears WidgetInstance. */
	UE_API void ClearWidgetInstance();

	UE_API const UUserWidget* GetWidgetCDO() const;
	UE_API const UUserWidget* GetWidgetCDOForSlot(const FName InSlotName) const;

	UE_API const FPreviewableWidgetVariant& GetWidgetType() const;
	UE_API void SetWidgetType(const FPreviewableWidgetVariant& InWidget);

	UE_API const TMap<FName, FPreviewableWidgetVariant>& GetSlotWidgetTypes() const;
	UE_API void SetSlotWidgetTypes(const TMap<FName, FPreviewableWidgetVariant>& InWidgets);

	UE_API const bool GetbShouldOverrideWidgetSize() const;
	UE_API void SetbShouldOverrideWidgetSize(bool InOverride);

	UE_API const FVector2D GetOverriddenWidgetSize() const;
	UE_API void SetOverriddenWidgetSize(FVector2D InWidgetSize);

	UE_API bool ShouldDefaultConstructViewmodels() const;
	UE_API void SetDefaultConstructViewmodels(bool bInDefaultConstruct);

protected:
	UE_API virtual void PostLoad() override;

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API void OnWidgetBlueprintChanged(UBlueprint* InBlueprint);

	/** Misc. functionality to perform after a widget assignment is changed. */
	UE_API void UpdateWidgets();

	/** Creates a new WidgetInstance, replacing the current one if it exists. */
	UE_API UUserWidget* CreateWidgetInstance(UWorld* InWorld);

	UE_API void CleanupReferences();

	/** Returns slot names not already occupied in SlotWidgets. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UE_API TArray<FName> GetAvailableWidgetSlotNames();

private:
	/** Widget to preview. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Widget", DisplayName = "Widget", meta = (AllowPrivateAccess = "true", ShowOnlyInnerProperties))
	FPreviewableWidgetVariant WidgetType;

	/** Widget per-slot, if WidgetType has any. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Widget", DisplayName = "Slot Widgets", meta = (AllowPrivateAccess = "true", GetKeyOptions = "GetAvailableWidgetSlotNames", ShowOnlyInnerProperties))
	TMap<FName, FPreviewableWidgetVariant> SlotWidgetTypes;

	/** Widget Custom Size Override */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Widget", DisplayName = "Override Widget Size", meta = (AllowPrivateAccess = "true", InlineEditConditionToggle))
	bool bShouldOverrideWidgetSize;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Widget", DisplayName = "Widget Size", meta = (AllowPrivateAccess = "true", EditCondition="bShouldOverrideWidgetSize"))
	FVector2D OverriddenWidgetSize;

	/** When this preview is opened in the previewer, automatically construct any missing viewmodels. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = ShouldDefaultConstructViewmodels, Setter = SetDefaultConstructViewmodels, Category = "Widget", meta = (AllowPrivateAccess = "true"))
	bool bDefaultConstructViewmodels;

	UPROPERTY(DuplicateTransient)
	TObjectPtr<UUserWidget> WidgetInstance;

	TSharedPtr<SWidget> SlateWidgetInstance;

	/** Slot names available in WidgetType (if any). */
	UPROPERTY(Transient)
	TArray<FName> SlotNameCache;

	/** Widgets here should be checked for validity when a new one is assigned, to allow tear-down functionality. */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<const UUserWidget>> WidgetReferenceCache;

	FOnWidgetChanged OnWidgetChangedDelegate;
};

#undef UE_API
