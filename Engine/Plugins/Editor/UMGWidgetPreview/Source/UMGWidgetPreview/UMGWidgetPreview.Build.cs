// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UMGWidgetPreview : ModuleRules
{
	public UMGWidgetPreview(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AdvancedPreviewScene",
				"AssetDefinition",
				"BlueprintGraph",
				"ContentBrowser",
				"Core",
				"CoreUObject",
                "DataValidation",
				"EditorSubsystem",
				"Engine",
				"FieldNotification",
				"InputCore",
				"MessageLog",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UMG",
				"UMGEditor",
				"UnrealEd",
			});
	}
}
